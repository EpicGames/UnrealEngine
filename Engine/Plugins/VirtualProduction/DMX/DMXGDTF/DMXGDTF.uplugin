{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX GDTF",
	"Description": "Implementation of the GDTF standard using Unreal Engine types",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "DMXGDTF",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXGDTFTests",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXZip",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
