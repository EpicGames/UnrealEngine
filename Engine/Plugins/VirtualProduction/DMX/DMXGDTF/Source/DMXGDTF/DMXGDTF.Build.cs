// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXGDTF : ModuleRules
{
	public DMXGDTF(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DMXZip",
				"Engine",
				"XmlParser"
			}
		);
	}
}
