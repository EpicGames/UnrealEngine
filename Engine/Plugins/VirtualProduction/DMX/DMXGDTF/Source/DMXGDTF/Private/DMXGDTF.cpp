// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXGDTF.h"

#include "DMXZipper.h"
#include "GDTF/DMXGDTFDescription.h"
#include "Misc/FileHelper.h"
#include "XmlFile.h"

void UDMXGDTF::InitializeFromData(const TArray64<uint8>& Data)
{
	using namespace UE::DMX::GDTF;

	const TSharedRef<FDMXZipper> Zip = MakeShared<FDMXZipper>();
	if (!Zip->LoadFromData(Data))
	{
		return;
	}

	TArray64<uint8> DescriptionXmlData;
	if (!Zip->GetFileContent(TEXT("Description.xml"), DescriptionXmlData))
	{
		return;
	}

	FString DescriptionXmlString;
	FFileHelper::BufferToString(DescriptionXmlString, DescriptionXmlData.GetData(), DescriptionXmlData.Num());

	const TSharedRef<FXmlFile> XmlFile = MakeShared<FXmlFile>();
	if (!XmlFile->LoadFile(DescriptionXmlString, EConstructMethod::ConstructFromBuffer))
	{
		return;
	}

	Description = MakeShared<FDMXGDTFDescription>();
	Description->InitializeFromDescriptionXml(XmlFile);
}

void UDMXGDTF::InitializeFromFixtureType(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
{
	Description = MakeShared<FDMXGDTFDescription>();
	Description->InitializeFromFixtureType(InFixtureType);
}

TSharedPtr<FXmlFile> UDMXGDTF::ExportAsXml() const
{
	return Description->ExportAsXml();
}
