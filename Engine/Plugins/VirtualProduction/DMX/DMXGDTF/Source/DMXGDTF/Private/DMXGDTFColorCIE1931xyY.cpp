// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXGDTFColorCIE1931xyY.h"

FString FDMXGDTFColorCIE1931xyY::ToString() const
{
	constexpr int32 SignificantDigits = 6;
	const FString XString = FString::SanitizeFloat(X, SignificantDigits);
	const FString YString = FString::SanitizeFloat(Y, SignificantDigits);
	const FString ZString = FString::SanitizeFloat(YY, SignificantDigits);

	return FString::Printf(TEXT("%s,%s,%s"), *XString, *YString, *ZString);
}
