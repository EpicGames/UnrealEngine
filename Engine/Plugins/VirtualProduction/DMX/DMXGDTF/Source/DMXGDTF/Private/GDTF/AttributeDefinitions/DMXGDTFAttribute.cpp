// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/AttributeDefinitions/DMXGDTFAttribute.h"

#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"
#include "GDTF/AttributeDefinitions/DMXGDTFPhysicalUnit.h"
#include "GDTF/AttributeDefinitions/DMXGDTFSubphysicalUnit.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFAttribute::FDMXGDTFAttribute(const TSharedRef<FDMXGDTFAttributeDefinitions>& InAttributeDefinitions)
		: OuterAttributeDefinitions(InAttributeDefinitions)
	{}

	void FDMXGDTFAttribute::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Pretty"), Pretty)
			.GetAttribute(TEXT("PhysicalUnit"), PhysicalUnit)
			.GetAttribute(TEXT("ActivationGroup"), ActivationGroup)
			.GetAttribute(TEXT("Feature"), Feature)
			.GetAttribute(TEXT("MainAttribute"), MainAttribute)
			.GetAttribute(TEXT("Color"), Color)
			.CreateChildren(TEXT("SubphysicalUnit"), SubpyhsicalUnitArray);
	}

	FXmlNode* FDMXGDTFAttribute::CreateXmlNode(FXmlNode& Parent)
	{
		const FString DefaultActivationGroup = TEXT("");
		const FString DefaultMainAttribute = TEXT("");
		const FDMXGDTFColorCIE1931xyY DefaultColor = { 0.f, 0.f, 0.f };

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Pretty"), Pretty)
			.SetAttribute(TEXT("PhysicalUnit"), PhysicalUnit)
			.SetAttribute(TEXT("ActivationGroup"), ActivationGroup, DefaultActivationGroup)
			.SetAttribute(TEXT("Feature"), Feature)
			.SetAttribute(TEXT("MainAttribute"), MainAttribute, DefaultMainAttribute)
			.SetAttribute(TEXT("Color"), Color, DefaultColor)
			.AppendChildren(TEXT("SubphysicalUnit"), SubpyhsicalUnitArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFActivationGroup> FDMXGDTFAttribute::ResolveActivationGroup() const
	{
		if (const TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions = OuterAttributeDefinitions.Pin())
		{
			return AttributeDefinitions->FindActivationGroup(ActivationGroup);
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFFeature> FDMXGDTFAttribute::ResolveFeature() const
	{
		if (const TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions = OuterAttributeDefinitions.Pin())
		{
			TArray<FString> Link;
			Feature.ParseIntoArray(Link, TEXT("."));
			if (Link.Num() == 2)
			{
				return AttributeDefinitions->FindFeature(Link[0], Link[1]);
			}
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFAttribute> FDMXGDTFAttribute::ResolveMainAttribute() const
	{
		if (const TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions = OuterAttributeDefinitions.Pin())
		{
			return AttributeDefinitions->FindAttribute(MainAttribute);
		}

		return nullptr;
	}
}
