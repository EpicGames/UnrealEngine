// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"

#include "Algo/Find.h"
#include "GDTF/AttributeDefinitions/DMXGDTFActivationGroup.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttribute.h"
#include "GDTF/AttributeDefinitions/DMXGDTFFeature.h"
#include "GDTF/AttributeDefinitions/DMXGDTFFeatureGroup.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFAttributeDefinitions::FDMXGDTFAttributeDefinitions(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
	{}

	void FDMXGDTFAttributeDefinitions::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateChildCollection(TEXT("ActivationGroups"), TEXT("ActivationGroup"), ActivationGroups)
			.CreateChildCollection(TEXT("FeatureGroups"), TEXT("FeatureGroup"), FeatureGroups)
			.CreateChildCollection(TEXT("Attributes"), TEXT("Attribute"), Attributes);
	}

	FXmlNode* FDMXGDTFAttributeDefinitions::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendChildCollection(TEXT("ActivationGroups"), TEXT("ActivationGroup"), ActivationGroups)
			.AppendChildCollection(TEXT("FeatureGroups"), TEXT("FeatureGroup"), FeatureGroups)
			.AppendChildCollection(TEXT("Attributes"), TEXT("Attribute"), Attributes);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFActivationGroup> FDMXGDTFAttributeDefinitions::FindActivationGroup(const FString& ActivationGroupName) const
	{
		if (const TSharedPtr<FDMXGDTFActivationGroup>* ActivationGroupPtr = Algo::FindBy(ActivationGroups, ActivationGroupName, &FDMXGDTFActivationGroup::Name))
		{
			return *ActivationGroupPtr;
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFAttribute> FDMXGDTFAttributeDefinitions::FindAttribute(const FString& AttributeName) const
	{
		if (const TSharedPtr<FDMXGDTFAttribute>* AttributePtr = Algo::FindBy(Attributes, AttributeName, &FDMXGDTFAttribute::Name))
		{
			return *AttributePtr;
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFFeature> FDMXGDTFAttributeDefinitions::FindFeature(const FString& FeatureGroupName, const FString& FeatureName) const
	{
		if (const TSharedPtr<FDMXGDTFFeatureGroup>* FeatureGroupPtr = Algo::FindBy(FeatureGroups, FeatureGroupName, &FDMXGDTFFeatureGroup::Name))
		{
			if (const TSharedPtr<FDMXGDTFFeature>* FeaturePtr = Algo::FindBy((*FeatureGroupPtr)->FeatureArray, FeatureName, &FDMXGDTFFeature::Name))
			{
				return *FeaturePtr;
			}
		}
		
		return nullptr;
	}
}
