// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/AttributeDefinitions/DMXGDTFFeature.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFFeature::FDMXGDTFFeature(const TSharedRef<FDMXGDTFFeatureGroup>& InFeatureGroup)
		: OuterFeatureGroup(InFeatureGroup)
	{}

	void FDMXGDTFFeature::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name);
	}
	
	FXmlNode* FDMXGDTFFeature::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
