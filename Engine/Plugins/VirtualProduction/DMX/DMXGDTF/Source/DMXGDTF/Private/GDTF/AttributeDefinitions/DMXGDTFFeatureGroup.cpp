// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/AttributeDefinitions/DMXGDTFFeatureGroup.h"

#include "GDTF/AttributeDefinitions/DMXGDTFFeature.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFFeatureGroup::FDMXGDTFFeatureGroup(const TSharedRef<FDMXGDTFAttributeDefinitions>& InAttributeDefinitions)
		: OuterAttributeDefinitions(InAttributeDefinitions)
	{}

	void FDMXGDTFFeatureGroup::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Pretty"), Pretty)
			.CreateChildren(TEXT("Feature"), FeatureArray);
	}

	FXmlNode* FDMXGDTFFeatureGroup::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Pretty"), Pretty)
			.AppendChildren(TEXT("Feature"), FeatureArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
