// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/AttributeDefinitions/DMXGDTFSubphysicalUnit.h"

#include "GDTF/AttributeDefinitions/DMXGDTFPhysicalUnit.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFSubphysicalUnit::FDMXGDTFSubphysicalUnit(const TSharedRef<FDMXGDTFAttribute>& InAttribute)
		: OuterAttribute(InAttribute)
	{}

	void FDMXGDTFSubphysicalUnit::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Type"), Type)
			.GetAttribute(TEXT("PhysicalUnit"), PhysicalUnit)
			.GetAttribute(TEXT("PhysicalFrom"), PhysicalFrom)
			.GetAttribute(TEXT("PhysicalTo"), PhysicalTo);
	}

	FXmlNode* FDMXGDTFSubphysicalUnit::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Type"), Type)
			.SetAttribute(TEXT("PhysicalUnit"), PhysicalUnit)
			.SetAttribute(TEXT("PhysicalFrom"), PhysicalFrom)
			.SetAttribute(TEXT("PhysicalTo"), PhysicalTo);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
