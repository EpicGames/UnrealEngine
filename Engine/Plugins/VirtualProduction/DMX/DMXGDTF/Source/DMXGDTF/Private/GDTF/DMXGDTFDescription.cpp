// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXGDTFDescription.h"

#include "DMXGDTFLog.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "Misc/MessageDialog.h"
#include "XmlFile.h"
#include "XmlNode.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFDescription::InitializeFromDescriptionXml(const TSharedRef<FXmlFile>& DescriptionXml)
	{
		// Parse Data Version
		FXmlNode* RootNode = DescriptionXml->GetRootNode();
		if (!RootNode)
		{
			UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF file. Description node is invalid.");
			return;
		}

		constexpr TCHAR DataVersionAttributeName[] = TEXT("DataVersion");
		const FString DataVersionString = RootNode->GetAttribute(DataVersionAttributeName);

		TArray<FString> MajorAndMinorVerison;
		DataVersionString.ParseIntoArray(MajorAndMinorVerison, TEXT("."), true);

		int32 MajorVersion;
		if (!MajorAndMinorVerison.IsValidIndex(0) ||
			!LexTryParseString<int32>(MajorVersion, *MajorAndMinorVerison[0]))
		{
			UE_LOGF(LogDMXGDTF, Warning, "Invalid GDTF Description.xml. Major Version is not numerical or not set.");
			return;
		}

		int32 MinorVersion;
		if (!MajorAndMinorVerison.IsValidIndex(1) ||
			!LexTryParseString<int32>(MinorVersion, *MajorAndMinorVerison[1]))
		{
			UE_LOGF(LogDMXGDTF, Warning, "Invalid GDTF Description.xml. Minor Version is not numerical or not set.");
			return;
		}

		// Warn if Data Version is newer than the version supported by engine
		if (FDMXGDTFVersion::MajorVersion < MajorVersion ||
			(FDMXGDTFVersion::MajorVersion == MajorVersion && FDMXGDTFVersion::MinorVersion < MinorVersion))
		{
			const FText VerMajorText = FText::FromString(MajorAndMinorVerison[0]);
			const FText VerMinorText = FText::FromString(MajorAndMinorVerison[1]);
			const FText EngineMajorVerText = FText::FromString(FDMXGDTFVersion::GetMajorVersionAsString());
			const FText EngineMinorVerText = FText::FromString(FDMXGDTFVersion::GetMinorVersionAsString());

			const EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(NSLOCTEXT("UDMXGDTFDescription", "TryLoadGDTFWithVersionNewThanEngineDialog", "Warning: Version '{0}.{1}' of GDTF is newer than the GDTF Version supported by the Engine, '{2}.{3}'. Do you want to try to load the GDTF anyway (not recommended)?"), VerMajorText, VerMinorText, EngineMajorVerText, EngineMinorVerText));
			if (DialogResult == EAppReturnType::Yes)
			{
				return;
			}
		}

		// Create nodes recursively from here on
		constexpr TCHAR NodeName_FixtureType[] = TEXT("FixtureType");
		FXmlNode* FixtureTypeXmlNode = RootNode->FindChildNode(NodeName_FixtureType);
		if (FixtureTypeXmlNode)
		{
			FixtureType = MakeShared<FDMXGDTFFixtureType>();
			FixtureType->Initialize(*FixtureTypeXmlNode);
		}
	}

	void FDMXGDTFDescription::InitializeFromFixtureType(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
	{	
		FixtureType = InFixtureType;
	}

	TSharedPtr<FXmlFile> FDMXGDTFDescription::ExportAsXml() const
	{
		// Don't export if there's nothing to export
		if (!FixtureType.IsValid())
		{
			return nullptr;
		}

		// Create Xml File
		const FString Buffer = "<?xml version=\"1.0\" encoding=\"UTF - 8\" standalone=\"no\" ?>\n<GDTF>\n</GDTF>";

		TSharedRef<FXmlFile> XmlFile = MakeShared<FXmlFile>();
		const bool bCreatedNewFile = XmlFile->LoadFile(Buffer, EConstructMethod::ConstructFromBuffer);

		FXmlNode* RootNode = XmlFile->GetRootNode();
		if (!ensureAlwaysMsgf(bCreatedNewFile && RootNode, TEXT("Failed to create a GDTF description.xml. Cannot export GDTF.")))
		{
			return nullptr;
		}
		check(RootNode);

		// Version the Root Node
		constexpr TCHAR DataVersionAttributeName[] = TEXT("DataVersion");
		TArray<FXmlAttribute> Attributes =
		{
			FXmlAttribute(DataVersionAttributeName, FDMXGDTFVersion::GetAsString()),
		};
		RootNode->SetAttributes(Attributes);

		// Export Children
		FixtureType->CreateXmlNode(*RootNode);

		return XmlFile;
	}
}
