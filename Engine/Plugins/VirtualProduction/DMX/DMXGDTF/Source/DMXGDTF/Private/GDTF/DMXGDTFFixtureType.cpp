// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXGDTFFixtureType.h"

#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/FTPresets/DMXGDTFFTPreset.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Models/DMXGDTFModel.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFPhysicalDescriptions.h"
#include "GDTF/Protocols/DMXGDTFProtocols.h"
#include "GDTF/Revisions/DMXGDTFRevision.h"
#include "GDTF/Wheels/DMXGDTFWheel.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFFixtureType::Initialize(const FXmlNode& InXmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), InXmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("ShortName"), ShortName)
			.GetAttribute(TEXT("LongName"), LongName)
			.GetAttribute(TEXT("Manufacturer"), Manufacturer)
			.GetAttribute(TEXT("Description"), Description)
			.GetAttribute(TEXT("FixtureTypeID"), FixtureTypeID)
			.GetAttribute(TEXT("Thumbnail"), Thumbnail)
			.GetAttribute(TEXT("ThumbnailOffsetX"), ThumbnailOffsetX)
			.GetAttribute(TEXT("ThumbnailOffsetY"), ThumbnailOffsetY)
			.GetAttribute(TEXT("RefFT"), RefFT)
			.GetAttribute(TEXT("CanHaveChildren"), bCanHaveChildren,
				[](const FString& StringValue)
				{
					// Parse as boolean. If field is empty, default to true.
					return StringValue.IsEmpty() || StringValue == TEXT("Yes");
				})
			.CreateRequiredChild(TEXT("AttributeDefinitions"), AttributeDefinitions)
			.CreateChildCollection(TEXT("Wheels"), TEXT("Wheel"), Wheels)
			.CreateOptionalChild(TEXT("PhysicalDescriptions"), PhysicalDescriptions)
			.CreateChildCollection(TEXT("Models"), TEXT("Model"), Models)
			.CreateOptionalChild(TEXT("Geometries"), GeometryCollect)
			.CreateChildCollection(TEXT("DMXModes"), TEXT("DMXMode"), DMXModes)
			.CreateChildCollection(TEXT("Revisions"), TEXT("Revision"), Revisions)
			.CreateChildCollection(TEXT("FTPresets"), TEXT("FTPreset"), FTPresets)
			.CreateOptionalChild(TEXT("Protocols"), Protocols);
	}

	FXmlNode* FDMXGDTFFixtureType::CreateXmlNode(FXmlNode& Parent)
	{
		const int32 DefaultThumbnailOffsetX = 0;
		const int32 DefaultThumbnailOffsetY = 0;

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("ShortName"), ShortName)
			.SetAttribute(TEXT("LongName"), LongName)
			.SetAttribute(TEXT("Manufacturer"), Manufacturer)
			.SetAttribute(TEXT("Description"), Description)
			.SetAttribute(TEXT("FixtureTypeID"), FixtureTypeID)
			.SetAttribute(TEXT("Thumbnail"), Thumbnail)
			.SetAttribute(TEXT("ThumbnailOffsetX"), ThumbnailOffsetX, DefaultThumbnailOffsetX)
			.SetAttribute(TEXT("ThumbnailOffsetY"), ThumbnailOffsetY, DefaultThumbnailOffsetY)
			.SetAttribute(TEXT("RefFT"), RefFT)
			.SetAttribute(TEXT("CanHaveChildren"), bCanHaveChildren ? TEXT("Yes") : TEXT("No"))
			.AppendRequiredChild(TEXT("AttributeDefinitions"), AttributeDefinitions)
			.AppendChildCollection(TEXT("Wheels"), TEXT("Wheel"), Wheels)
			.AppendOptionalChild(TEXT("PhysicalDescriptions"), PhysicalDescriptions)
			.AppendChildCollection(TEXT("Models"), TEXT("Model"), Models)
			.AppendOptionalChild(TEXT("Geometries"), GeometryCollect)
			.AppendChildCollection(TEXT("DMXModes"), TEXT("DMXMode"), DMXModes)
			.AppendChildCollection(TEXT("FTPresets"), TEXT("FTPreset"), FTPresets)
			.AppendChildCollection(TEXT("Revisions"), TEXT("Revision"), Revisions)
			.AppendOptionalChild(TEXT("Protocols"), Protocols);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
