// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXGDTFVersion.h"

namespace UE::DMX::GDTF
{
	const FString FDMXGDTFVersion::GetMajorVersionAsString()
	{
		return FString::FromInt(FDMXGDTFVersion::MajorVersion);
	}

	const FString FDMXGDTFVersion::GetMinorVersionAsString()
	{
		return FString::FromInt(FDMXGDTFVersion::MinorVersion);
	}

	const FString FDMXGDTFVersion::GetAsString()
	{
		return GetMajorVersionAsString() + TEXT(".") + GetMinorVersionAsString();
	}
}
