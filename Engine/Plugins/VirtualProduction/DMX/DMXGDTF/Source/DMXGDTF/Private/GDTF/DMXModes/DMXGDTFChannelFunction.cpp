// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFChannelFunction.h"

#include "Algo/Find.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFChannelSet.h"
#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/DMXModes/DMXGDTFLogicalChannel.h"
#include "GDTF/DMXModes/DMXGDTFSubchannelSet.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFColorSpace.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFDMXProfile.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFEmitter.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFFilter.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFGamut.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFPhysicalDescriptions.h"
#include "GDTF/Wheels/DMXGDTFWheel.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFChannelFunction::FDMXGDTFChannelFunction(const TSharedRef<FDMXGDTFLogicalChannel>& InLogicalChannel)
		: OuterLogicalChannel(InLogicalChannel)
	{}

	void FDMXGDTFChannelFunction::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Attribute"), Attribute)
			.GetAttribute(TEXT("OriginalAttribute"), OriginalAttribute)
			.GetAttribute(TEXT("DMXFrom"), DMXFrom)
			.GetAttribute(TEXT("Default"), Default, this, &FDMXGDTFChannelFunction::ParseDefault, &XmlNode)
			.GetAttribute(TEXT("PhysicalFrom"), PhysicalFrom)
			.GetAttribute(TEXT("PhysicalTo"), PhysicalTo)
			.GetAttribute(TEXT("RealFade"), RealFade)
			.GetAttribute(TEXT("RealAcceleration"), RealAcceleration)
			.GetAttribute(TEXT("Wheel"), Wheel)
			.GetAttribute(TEXT("Emitter"), Emitter)
			.GetAttribute(TEXT("Filter"), Filter)
			.GetAttribute(TEXT("ColorSpace"), ColorSpace)
			.GetAttribute(TEXT("Gamut"), Gamut)
			.GetAttribute(TEXT("ModeMaster"), ModeMaster)
			.GetAttribute(TEXT("ModeFrom"), ModeFrom)
			.GetAttribute(TEXT("ModeTo"), ModeTo)
			.GetAttribute(TEXT("DMXProfile"), DMXProfile)
			.GetAttribute(TEXT("Min"), Min)
			.GetAttribute(TEXT("Max"), Max)
			.GetAttribute(TEXT("CustomName"), CustomName)
			.CreateChildren(TEXT("ChannelSet"), ChannelSetArray)
			.CreateChildren(TEXT("SubchannelSet"), SubchannelSetArray);

		// As per specs, Min and Max default the same as PhysicalFrom and PhysicalTo
		if (Min != PhysicalFrom)
		{
			Min = PhysicalFrom;
		}

		if (Max != PhysicalTo)
		{
			Max = PhysicalTo;
		}
	}

	FXmlNode* FDMXGDTFChannelFunction::CreateXmlNode(FXmlNode& Parent)
	{
		const FString DefaultLink = TEXT("");
		const FDMXGDTFDMXValue DefaultDefault = FDMXGDTFDMXValue(0);
		const FDMXGDTFDMXValue DefaultModeFrom = FDMXGDTFDMXValue(0);
		const FDMXGDTFDMXValue DefaultModeTo = FDMXGDTFDMXValue(0);
		const float DefaultMin = PhysicalFrom;
		const float DefaultMax = PhysicalTo;
		const FString DefaultCustomName = TEXT("");

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Attribute"), Attribute)
			.SetAttribute(TEXT("OriginalAttribute"), OriginalAttribute)
			.SetAttribute(TEXT("DMXFrom"), DMXFrom)
			.SetAttribute(TEXT("Default"), Default)
			.SetAttribute(TEXT("PhysicalFrom"), PhysicalFrom)
			.SetAttribute(TEXT("PhysicalTo"), PhysicalTo)
			.SetAttribute(TEXT("RealFade"), RealFade)
			.SetAttribute(TEXT("RealAcceleration"), RealAcceleration)
			.SetAttribute(TEXT("Wheel"), Wheel, DefaultLink)
			.SetAttribute(TEXT("Emitter"), Emitter, DefaultLink)
			.SetAttribute(TEXT("Filter"), Filter, DefaultLink)
			.SetAttribute(TEXT("ColorSpace"), ColorSpace, DefaultLink)
			.SetAttribute(TEXT("Gamut"), Gamut, DefaultLink)
			.SetAttribute(TEXT("ModeMaster"), ModeMaster, DefaultLink)
			.SetAttribute(TEXT("ModeFrom"), ModeFrom, DefaultModeFrom)
			.SetAttribute(TEXT("ModeTo"), ModeTo, DefaultModeTo)
			.SetAttribute(TEXT("DMXProfile"), DMXProfile, DefaultLink)
			.SetAttribute(TEXT("Min"), Min, DefaultMin)
			.SetAttribute(TEXT("Max"), Max, DefaultMax)
			.SetAttribute(TEXT("CustomName"), CustomName, DefaultCustomName)
			.AppendChildren(TEXT("ChannelSet"), ChannelSetArray)
			.AppendChildren(TEXT("SubchannelSet"), SubchannelSetArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFAttribute> FDMXGDTFChannelFunction::ResolveAttribute() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions = FixtureType.IsValid() ? FixtureType->AttributeDefinitions : nullptr;
		
		return AttributeDefinitions.IsValid() ? AttributeDefinitions->FindAttribute(Attribute) : nullptr;
	}

	TSharedPtr<FDMXGDTFWheel> FDMXGDTFChannelFunction::ResolveWheel() const
	{
		if (const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin())
		{
			const TSharedPtr<FDMXGDTFWheel>* WheelPtr = Algo::FindBy(FixtureType->Wheels, Wheel, &FDMXGDTFWheel::Name);
			if (WheelPtr)
			{
				return *WheelPtr;
			}
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFEmitter> FDMXGDTFChannelFunction::ResolveEmitter() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			if (const TSharedPtr<FDMXGDTFEmitter>* EmitterPtr = Algo::FindBy(PhysicalDescriptions->Emitters, Emitter, &FDMXGDTFEmitter::Name))
			{
				return *EmitterPtr;
			}
		}
		return nullptr;
	}

	TSharedPtr<FDMXGDTFFilter> FDMXGDTFChannelFunction::ResolveFilter() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			if (const TSharedPtr<FDMXGDTFFilter>* FilterPtr = Algo::FindBy(PhysicalDescriptions->Filters, Filter, &FDMXGDTFFilter::Name))
			{
				return *FilterPtr;
			}
		}
		return nullptr;
	}

	TSharedPtr<FDMXGDTFColorSpace> FDMXGDTFChannelFunction::ResolveColorSpace() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			return PhysicalDescriptions->ColorSpaces;
		}
		return nullptr;
	}

	TSharedPtr<FDMXGDTFGamut> FDMXGDTFChannelFunction::ResolveGamut() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			if (const TSharedPtr<FDMXGDTFGamut>* GamutPtr = Algo::FindBy(PhysicalDescriptions->Gamuts, Gamut, &FDMXGDTFGamut::Name))
			{
				return *GamutPtr;
			}
		}
		return nullptr;
	}

	TSharedPtr<FDMXGDTFDMXProfile> FDMXGDTFChannelFunction::ResolveDMXProfile() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			if (const TSharedPtr<FDMXGDTFDMXProfile>* DMXProfilePtr = Algo::FindBy(PhysicalDescriptions->DMXProfiles, DMXProfile, &FDMXGDTFDMXProfile::Name))
			{
				return *DMXProfilePtr;
			}
		}
		return nullptr;
	}

	void FDMXGDTFChannelFunction::ResolveModeMaster(TSharedPtr<FDMXGDTFDMXChannel>& OutDMXChannel, TSharedPtr<FDMXGDTFChannelFunction>& OutChannelFunction) const
	{
		const TSharedPtr<FDMXGDTFLogicalChannel> LogicalChannel = OuterLogicalChannel.Pin();
		const TSharedPtr<FDMXGDTFDMXChannel> DMXChannel = LogicalChannel.IsValid() ? LogicalChannel->OuterDMXChannel.Pin() : nullptr;
		const TSharedPtr<FDMXGDTFDMXMode> DMXMode = DMXChannel.IsValid() ? DMXChannel->OuterDMXMode.Pin() : nullptr;
		if (DMXMode.IsValid())
		{
			return DMXMode->ResolveChannel(ModeMaster, OutDMXChannel, OutChannelFunction);
		}
	}

	void FDMXGDTFChannelFunction::ResolveModePrimary(TSharedPtr<FDMXGDTFDMXChannel>& OutDMXChannel, TSharedPtr<FDMXGDTFChannelFunction>& OutChannelFunction) const
	{
		ResolveModeMaster(OutDMXChannel, OutChannelFunction);
	}
	
	FDMXGDTFDMXValue FDMXGDTFChannelFunction::ParseDefault(const FString& Value, const FXmlNode* XmlNode) const
	{
		if (!Value.IsEmpty())
		{
			return FDMXGDTFDMXValue(*Value);
		}
		else if(Algo::FindBy(XmlNode->GetAttributes(), TEXT("Default"), &FXmlAttribute::GetValue) == nullptr)
		{						
			// Try to read the value from the DXM channel instead, as it was defined in GDTF 1.0

			const TSharedPtr<FDMXGDTFDMXChannel> DMXChannel = OuterLogicalChannel.IsValid() ? OuterLogicalChannel.Pin()->OuterDMXChannel.Pin() : nullptr;
			if (ensureMsgf(DMXChannel.IsValid(), TEXT("Invalid node, node does not reside in a valid DMX channel.")))
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				return DMXChannel->Default;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		return FDMXGDTFDMXValue();
	}
}
