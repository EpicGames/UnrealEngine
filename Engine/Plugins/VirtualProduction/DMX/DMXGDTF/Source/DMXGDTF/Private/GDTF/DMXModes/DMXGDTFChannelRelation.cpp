// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFChannelRelation.h"

#include "Algo/Find.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFChannelRelation::FDMXGDTFChannelRelation(const TSharedRef<FDMXGDTFDMXMode>& InDMXMode)
		: OuterDMXMode(InDMXMode)
	{}

	void FDMXGDTFChannelRelation::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Master"), Master)
			.GetAttribute(TEXT("Follower"), Follower)
			.GetAttribute(TEXT("Type"), Type);
	}

	FXmlNode* FDMXGDTFChannelRelation::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Master"), Master)
			.SetAttribute(TEXT("Follower"), Follower)
			.SetAttribute(TEXT("Type"), Type);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFDMXChannel> FDMXGDTFChannelRelation::ResolveMaster() const
	{
		if (const TSharedPtr<FDMXGDTFDMXMode> DMXMode = OuterDMXMode.Pin())
		{
			TSharedPtr<FDMXGDTFDMXChannel> DMXChannel;
			TSharedPtr<FDMXGDTFChannelFunction> Dummy;
			DMXMode->ResolveChannel(Master, DMXChannel, Dummy);

			return DMXChannel;
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFChannelFunction> FDMXGDTFChannelRelation::ResolveFollower() const
	{
		if (const TSharedPtr<FDMXGDTFDMXMode> DMXMode = OuterDMXMode.Pin())
		{
			TSharedPtr<FDMXGDTFDMXChannel> Dummy;
			TSharedPtr<FDMXGDTFChannelFunction> ChannelFunction;
			DMXMode->ResolveChannel(Master, Dummy, ChannelFunction);

			return ChannelFunction;
		}

		return nullptr;
	}
}
