// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFChannelSet.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFChannelSet::FDMXGDTFChannelSet(const TSharedRef<FDMXGDTFChannelFunction>& InChannelFunction)
		: OuterChannelFunction(InChannelFunction)
	{}

	void FDMXGDTFChannelSet::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("DMXFrom"), DMXFrom)
			.GetAttribute(TEXT("PhysicalFrom"), PhysicalFrom)
			.GetAttribute(TEXT("PhysicalTo"), PhysicalTo)
			.GetAttribute(TEXT("WheelSlotIndex"), WheelSlotIndex);
	}

	FXmlNode* FDMXGDTFChannelSet::CreateXmlNode(FXmlNode& Parent)
	{
		const float DefaultPhysicalFrom = 0.f;
		const float DefaultPhysicalTo = 1.f;

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("DMXFrom"), DMXFrom)
			.SetAttribute(TEXT("PhysicalFrom"), PhysicalFrom, DefaultPhysicalFrom)
			.SetAttribute(TEXT("PhysicalTo"), PhysicalTo, DefaultPhysicalTo)
			.SetAttribute(TEXT("WheelSlotIndex"), WheelSlotIndex);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
