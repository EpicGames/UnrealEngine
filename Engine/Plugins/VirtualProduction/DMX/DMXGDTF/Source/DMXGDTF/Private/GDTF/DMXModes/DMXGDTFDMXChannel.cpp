// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"

#include "DMXGDTFLog.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFChannelFunction.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/DMXModes/DMXGDTFDMXValue.h"
#include "GDTF/DMXModes/DMXGDTFLogicalChannel.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Geometries/DMXGDTFGeometryReference.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFDMXChannel::FDMXGDTFDMXChannel(const TSharedRef<FDMXGDTFDMXMode>& InDMXMode)
		: OuterDMXMode(InDMXMode)
	{}

	void FDMXGDTFDMXChannel::Initialize(const FXmlNode& XmlNode)
	{
		constexpr bool bOnlyTopLevelGeoemtry = false;

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("DMXBreak"), DMXBreak)
			.GetAttribute(TEXT("Offset"), Offset, this, &FDMXGDTFDMXChannel::ParseOffset)
			.GetAttribute(TEXT("InitialFunction"), InitialFunction)
			.GetAttribute(TEXT("Highlight"), Highlight)
			.GetAttribute(TEXT("Geometry"), Geometry)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			.GetAttribute(TEXT("Default"), Default) // Deprecated with GDTF 1.1, but still initialized so old GDTFs can be supported. See DMXGDTFChannelFunction for the upgrade path.
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			.CreateChildren(TEXT("LogicalChannel"), LogicalChannelArray);
	}

	FXmlNode* FDMXGDTFDMXChannel::CreateXmlNode(FXmlNode& Parent)
	{
		const FString DefaultInitialFunction = TEXT("");

		FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this);

		// Write special value overwrite if the DMXBreak is < 1
		if (DMXBreak < 0)
		{
			ChildBuilder
				.SetAttribute(TEXT("DMXBreak"), TEXT("Overwrite"));
		}
		else
		{
			ChildBuilder
				.SetAttribute(TEXT("DMXBreak"), DMXBreak);
		}

		ChildBuilder
			.SetAttribute(TEXT("Offset"), Offset)
			.SetAttribute(TEXT("InitialFunction"), InitialFunction, DefaultInitialFunction)
			.SetAttribute(TEXT("Geometry"), Geometry)
			.AppendChildren(TEXT("LogicalChannel"), LogicalChannelArray);

		// Set special value "None" for highlight if Highlight is not set
		if (Highlight.IsSet())
		{
			ChildBuilder
				.SetAttribute(TEXT("Highlight"), Highlight);
		}
		else
		{
			ChildBuilder
				.SetAttribute(TEXT("Highlight"), TEXT("None"));
		}

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFChannelFunction> FDMXGDTFDMXChannel::ResolveInitialFunction() const
	{
		if (InitialFunction.IsEmpty() && !LogicalChannelArray.IsEmpty() && LogicalChannelArray[0].IsValid())
		{
			// Default value is the first channel function of the first logical function of this DMX channel.
			return LogicalChannelArray[0]->ChannelFunctionArray.IsEmpty() ? nullptr : LogicalChannelArray[0]->ChannelFunctionArray[0];
		}

		if (const TSharedPtr<FDMXGDTFDMXMode> DMXMode = OuterDMXMode.Pin())
		{
			TSharedPtr<FDMXGDTFDMXChannel> Dummy;
			TSharedPtr<FDMXGDTFChannelFunction> ChannelFunction;
			DMXMode->ResolveChannel(InitialFunction, Dummy, ChannelFunction);

			if (ChannelFunction.IsValid())
			{
				return ChannelFunction;
			}
			else if (!LogicalChannelArray.IsEmpty() && LogicalChannelArray[0].IsValid())
			{
				// As per specs, the first channel function if no initial function is specified
				return LogicalChannelArray[0]->ChannelFunctionArray.IsEmpty() ? nullptr : LogicalChannelArray[0]->ChannelFunctionArray[0];
			}
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFGeometry> FDMXGDTFDMXChannel::ResolveGeometry() const
	{
		const TSharedPtr<FDMXGDTFDMXMode> DMXMode = OuterDMXMode.Pin();
		const TSharedPtr<FDMXGDTFGeometry> TopLevelGeometry = DMXMode.IsValid() ? DMXMode->ResolveGeometry() : nullptr;
		const TSharedPtr<FDMXGDTFGeometry> GeometryNode = TopLevelGeometry.IsValid() ? TopLevelGeometry->FindGeometryByName(*Geometry.ToString()) : nullptr;

		return GeometryNode;
	}

	TArray<TSharedPtr<FDMXGDTFGeometryReference>> FDMXGDTFDMXChannel::ResolveGeometryReferences() const
	{
		const TSharedPtr<FDMXGDTFDMXMode> DMXMode = OuterDMXMode.Pin();
		const TSharedPtr<FDMXGDTFGeometry> TopLevelGeometry = DMXMode.IsValid() ? DMXMode->ResolveGeometry() : nullptr;

		TArray<TSharedPtr<FDMXGDTFGeometryReference>> AllGeometryReferences;
		if (TopLevelGeometry.IsValid())
		{
			TArray<TSharedPtr<FDMXGDTFGeometry>> Geometries;
			TopLevelGeometry->GetGeometriesRecursive(Geometries, AllGeometryReferences);
		}

		TArray<TSharedPtr<FDMXGDTFGeometryReference>> GeometryReferences;
		Algo::TransformIf(AllGeometryReferences, GeometryReferences, 
			[this](const TSharedPtr<FDMXGDTFGeometryReference>& GeometryReference)
			{
				return 
					GeometryReference.IsValid() && 
					GeometryReference->Geometry == Geometry;
			},
			[](const TSharedPtr<FDMXGDTFGeometryReference>& GeometryReference)
			{
				return GeometryReference;
			});

		return GeometryReferences;
	}

	TArray<uint32> FDMXGDTFDMXChannel::ParseOffset(const FString& GDTFString) const
	{
		TArray<FString> Substrings;
		GDTFString.ParseIntoArray(Substrings, TEXT(","));

		TArray<uint32> Result;
		for (const FString& Substring : Substrings)
		{
			uint32 Value;
			if (LexTryParseString(Value, *Substring))
			{
				Result.Add(Value);
			}
		}

		return Result;
	}
}
