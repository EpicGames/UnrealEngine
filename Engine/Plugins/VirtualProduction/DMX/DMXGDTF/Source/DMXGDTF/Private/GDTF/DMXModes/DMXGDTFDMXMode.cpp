// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFDMXMode.h"

#include "Algo/Find.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFChannelFunction.h"
#include "GDTF/DMXModes/DMXGDTFChannelRelation.h"
#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"
#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"
#include "GDTF/DMXModes/DMXGDTFFTMacro.h"
#include "GDTF/DMXModes/DMXGDTFLogicalChannel.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFDMXMode::FDMXGDTFDMXMode(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFDMXMode::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Description"), Description)
			.GetAttribute(TEXT("Geometry"), Geometry)
			.CreateChildCollection(TEXT("DMXChannels"), TEXT("DMXChannel"), DMXChannels)
			.CreateChildCollection(TEXT("Relations"), TEXT("Relation"), Relations)
			.CreateChildCollection(TEXT("FTMacros"), TEXT("FTMacro"), FTMacros);
	}

	FXmlNode* FDMXGDTFDMXMode::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Description"), Description)
			.SetAttribute(TEXT("Geometry"), Geometry)
			.AppendChildCollection(TEXT("DMXChannels"), TEXT("DMXChannel"), DMXChannels)
			.AppendChildCollection(TEXT("Relations"), TEXT("Relation"), Relations)
			.AppendChildCollection(TEXT("FTMacros"), TEXT("FTMacro"), FTMacros);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFGeometry> FDMXGDTFDMXMode::ResolveGeometry() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFGeometryCollect> GeometryCollect = FixtureType.IsValid() ? FixtureType->GeometryCollect : nullptr;
		const TSharedPtr<FDMXGDTFGeometry> GeometryNode = GeometryCollect.IsValid() ? GeometryCollect->FindGeometryByName(*Geometry.ToString()) : nullptr;

		return GeometryNode;
	}

	void FDMXGDTFDMXMode::ResolveChannel(const FString& Link, TSharedPtr<FDMXGDTFDMXChannel>& OutDMXChannel, TSharedPtr<FDMXGDTFChannelFunction>& OutChannelFunction) const
	{
		if (Link.IsEmpty())
		{
			return;
		}

		TArray<FString> LinkArray;
		Link.ParseIntoArray(LinkArray, TEXT("."));
		if (Link.IsEmpty())
		{
			return;
		}

		TArray<FString> GeometryAndAttribute;
		LinkArray[0].ParseIntoArray(GeometryAndAttribute, TEXT("_"));
		if (GeometryAndAttribute.Num() != 2 ||
			LinkArray[1] != GeometryAndAttribute[1])
		{
			return;
		}

		const FString& GeometryName = GeometryAndAttribute[0];
		const FString& AttributeName = GeometryAndAttribute[1];
		const FString& ChannelFunctionName = LinkArray[2];

		if (const TSharedPtr<FDMXGDTFDMXChannel>* DMXChannelPtr = Algo::FindBy(DMXChannels, GeometryName, &FDMXGDTFDMXChannel::Geometry))
		{
			if (const TSharedPtr<FDMXGDTFLogicalChannel>* LogicalChannelPtr = Algo::FindBy((*DMXChannelPtr)->LogicalChannelArray, AttributeName, &FDMXGDTFLogicalChannel::Attribute))
			{
				if (LinkArray.Num() == 1)
				{
					OutDMXChannel = *DMXChannelPtr;
				}
				else if (LinkArray.Num() == 3)
				{
					if (const TSharedPtr<FDMXGDTFChannelFunction>* ChannelFunctionPtr = Algo::FindBy((*LogicalChannelPtr)->ChannelFunctionArray, ChannelFunctionName, &FDMXGDTFChannelFunction::Name))
					{
						OutChannelFunction = *ChannelFunctionPtr;
					}
				}
			}
		}
	}
}
