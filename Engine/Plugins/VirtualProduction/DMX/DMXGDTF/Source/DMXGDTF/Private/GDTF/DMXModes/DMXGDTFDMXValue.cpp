// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFDMXValue.h"

#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFDMXValue::FDMXGDTFDMXValue(const TCHAR* InStringValue)
	{
		const FString StringValue = InStringValue;
		if (StringValue.IsEmpty() || StringValue == TEXT("None"))
		{
			return;
		}

		TArray<FString> Substrings;
		StringValue.ParseIntoArray(Substrings, TEXT("/"));

		if (Substrings.Num() != 2)
		{
			return;
		}

		if (!LexTryParseString(Value, *Substrings[0]))
		{
			return;
		}

		if (Substrings[1].EndsWith(TEXT("s")))
		{
			bByteMirroring = false;
			Substrings[1].RemoveFromEnd(TEXT("s"));
		}

		if (!LexTryParseString(NumBytes, *Substrings[1]))
		{
			return;
		}
	}

	FDMXGDTFDMXValue::FDMXGDTFDMXValue(const uint32 InValue, const int32 InNumBytes, const bool bInByteMirroring)
		: Value(InValue)
		, NumBytes(InNumBytes)
		, bByteMirroring(bInByteMirroring)
	{}

	bool FDMXGDTFDMXValue::Get(const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannel, uint32& OutValue) const
	{
		if (IsSet())
		{
			OutValue = 0;
			return false;
		}

		OutValue = GetChecked(InDMXChannel);
		return true;
	}

	uint32 FDMXGDTFDMXValue::GetChecked(const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannel) const
	{
		if (IsSet())
		{
			const uint8 WordSize = InDMXChannel->Offset.Num();

			if (WordSize == NumBytes)
			{
				return Value;
			}
			else if (bByteMirroring)
			{
				return static_cast<uint64>(Value) * GetMax(WordSize) / GetMax(NumBytes);
			}
			else
			{
				// Byte shift
				return Value << NumBytes * 8;
			}
		}
		return 0;
	}

	FString FDMXGDTFDMXValue::AsString() const
	{
		if (IsSet())
		{
			return bByteMirroring ?
				FString::Printf(TEXT("%u/%u"), Value, NumBytes) :
				FString::Printf(TEXT("%u/%us"), Value, NumBytes);
		}
		else
		{
			return TEXT("0/1");
		}
	}

	void FDMXGDTFDMXValue::Reset()
	{
		NumBytes = 0;
	}

	bool FDMXGDTFDMXValue::IsSet() const
	{
		return NumBytes != 0;
	}

	uint32 FDMXGDTFDMXValue::GetMax(uint8 WordSize) const
	{
		switch (WordSize)
		{
		case 1:
			return 0xFF;
		case 2:
			return 0xFFFF;
		case 3:
			return 0xFFFFFF;
		case 4:
			return 0xFFFFFFFF;
		};

		return 0;
	}
}
