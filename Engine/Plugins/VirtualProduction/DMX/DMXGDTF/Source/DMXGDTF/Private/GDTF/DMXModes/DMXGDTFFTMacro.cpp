// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFFTMacro.h"

#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/DMXModes/DMXGDTFMacroDMX.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFFTMacro::FDMXGDTFFTMacro(const TSharedRef<FDMXGDTFDMXMode>& InDMXMode)
		: OuterDMXMode(InDMXMode)
	{}

	void FDMXGDTFFTMacro::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("ChannelFunction"), ChannelFunction)
			.CreateChildren(TEXT("MacroDMX"), MacroDMXArray);
	}

	FXmlNode* FDMXGDTFFTMacro::CreateXmlNode(FXmlNode& Parent)
	{
		const FString DefaultChannelFunction = TEXT("");

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("ChannelFunction"), ChannelFunction, DefaultChannelFunction)
			.AppendChildren(TEXT("MacroDMX"), MacroDMXArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFChannelFunction> FDMXGDTFFTMacro::ResolveChannelFunction() const
	{
		if (const TSharedPtr<FDMXGDTFDMXMode> DMXMode = OuterDMXMode.Pin())
		{
			TSharedPtr<class FDMXGDTFDMXChannel> Dummy;
			TSharedPtr<FDMXGDTFChannelFunction> ResolvedChannelFunction;
			DMXMode->ResolveChannel(ChannelFunction, Dummy, ResolvedChannelFunction);

			return ResolvedChannelFunction;
		}

		return nullptr;
	}
}
