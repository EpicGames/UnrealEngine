// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFLogicalChannel.h"

#include "Algo/Find.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttribute.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFChannelFunction.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFLogicalChannel::FDMXGDTFLogicalChannel(const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannel)
		: OuterDMXChannel(InDMXChannel)
	{}

	void FDMXGDTFLogicalChannel::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Attribute"), Attribute)
			.GetAttribute(TEXT("Snap"), Snap, this, &FDMXGDTFLogicalChannel::ParseSnap)
			.GetAttribute(TEXT("Master"), Master, this, &FDMXGDTFLogicalChannel::ParseMaster)
			.GetAttribute(TEXT("MibFade"), MibFade)
			.GetAttribute(TEXT("DMXChangeTimeLimit"), DMXChangeTimeLimit)
			.CreateChildren(TEXT("ChannelFunction"), ChannelFunctionArray);
	}

	FXmlNode* FDMXGDTFLogicalChannel::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Attribute"), Attribute)
			.SetAttribute(TEXT("Snap"), Snap)
			.SetAttribute(TEXT("Master"), Master)
			.SetAttribute(TEXT("MibFade"), MibFade)
			.SetAttribute(TEXT("DMXChangeTimeLimit"), DMXChangeTimeLimit)
			.AppendChildren(TEXT("ChannelFunction"), ChannelFunctionArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFAttribute> FDMXGDTFLogicalChannel::ResolveAttribute() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions = FixtureType.IsValid() ? FixtureType->AttributeDefinitions : nullptr;
		if (AttributeDefinitions.IsValid())
		{
			const TSharedPtr<FDMXGDTFAttribute>* AttributePtr = Algo::FindBy(AttributeDefinitions->Attributes, Attribute, &FDMXGDTFAttribute::Name);
			if (AttributePtr)
			{
				return *AttributePtr;
			}
		}

		return nullptr;
	}

	EDMXGDTFLogicalChannelSnap FDMXGDTFLogicalChannel::ParseSnap(const FString& GDTFString) const
	{
		const FTopLevelAssetPath SnapEnumPath(TEXT("/Script/DMXGDTF"), TEXT("EDMXGDTFLogicalChannelSnap"));
		const UEnum* SnapUEnum = FindObject<UEnum>(SnapEnumPath);
		if (!ensureMsgf(SnapUEnum, TEXT("Cannot find enum for snap. Reflected enum no longer exists.")))
		{
			return EDMXGDTFLogicalChannelSnap::No;
		}

		// Find the corresponding enum value
		for (EDMXGDTFLogicalChannelSnap EnumElement : TEnumRange<EDMXGDTFLogicalChannelSnap>())
		{
			if (SnapUEnum->GetNameStringByValue((int64)EnumElement) == GDTFString)
			{
				return EnumElement;
			}
		}

		UE_LOGF(LogDMXGDTF, Warning, "Could not find definition for snap '%ls'.", *GDTFString);
		return EDMXGDTFLogicalChannelSnap::No;
	}

	EDMXGDTFLogicalChannelMaster FDMXGDTFLogicalChannel::ParseMaster(const FString& GDTFString) const
	{
		const FTopLevelAssetPath MasterEnumPath(TEXT("/Script/DMXGDTF"), TEXT("EDMXGDTFLogicalChannelMaster"));
		const UEnum* MasterUEnum = FindObject<UEnum>(MasterEnumPath);
		if (!ensureMsgf(MasterUEnum, TEXT("Cannot find enum for snap. Reflected enum no longer exists.")))
		{
			return EDMXGDTFLogicalChannelMaster::None;
		}

		// Find the corresponding enum value
		for (EDMXGDTFLogicalChannelMaster EnumElement : TEnumRange<EDMXGDTFLogicalChannelMaster>())
		{
			if (MasterUEnum->GetNameStringByValue((int64)EnumElement) == GDTFString)
			{
				return EnumElement;
			}
		}

		UE_LOGF(LogDMXGDTF, Warning, "Could not find definition for snap '%ls'.", *GDTFString);
		return EDMXGDTFLogicalChannelMaster::None;
	}
}
