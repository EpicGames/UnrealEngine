// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFMacroDMX.h"

#include "GDTF/DMXModes/DMXGDTFMacroDMXStep.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFMacroDMX::FDMXGDTFMacroDMX(const TSharedRef<FDMXGDTFFTMacro>& InFTMacro)
		: OuterFTMacro(InFTMacro)
	{}

	void FDMXGDTFMacroDMX::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateChildren(TEXT("MacroDMXStep"), MacroDMXStepArray);
	}

	FXmlNode* FDMXGDTFMacroDMX::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendChildren(TEXT("MacroDMXStep"), MacroDMXStepArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
