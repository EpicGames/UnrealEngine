// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFMacroDMXStep.h"

#include "GDTF/DMXModes/DMXGDTFMacroDMXValue.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFMacroDMXStep::FDMXGDTFMacroDMXStep(const TSharedRef<FDMXGDTFMacroDMX>& InMacroDMX)
		: OuterMacroDMX(InMacroDMX)
	{}

	void FDMXGDTFMacroDMXStep::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Duration"), Duration)
			.CreateChildren(TEXT("MacroDMXValue"), MacroDMXValueArray);
	}

	FXmlNode* FDMXGDTFMacroDMXStep::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Duration"), Duration)
			.AppendChildren(TEXT("MacroDMXValue"), MacroDMXValueArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
