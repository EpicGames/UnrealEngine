// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFMacroDMXValue.h"

#include "GDTF/DMXModes/DMXGDTFDMXChannel.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "GDTF/DMXModes/DMXGDTFFTMacro.h"
#include "GDTF/DMXModes/DMXGDTFMacroDMX.h"
#include "GDTF/DMXModes/DMXGDTFMacroDMXStep.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFMacroDMXValue::FDMXGDTFMacroDMXValue(const TSharedRef<FDMXGDTFMacroDMXStep>& InMacroDMXStep)
		: OuterMacroDMXStep(InMacroDMXStep)
	{}

	void FDMXGDTFMacroDMXValue::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("DMXValue"), DMXValue)
			.GetAttribute(TEXT("DMXChannel"), DMXChannel);
	}

	FXmlNode* FDMXGDTFMacroDMXValue::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("DMXValue"), DMXValue)
			.SetAttribute(TEXT("DMXChannel"), DMXChannel);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFDMXChannel> FDMXGDTFMacroDMXValue::ResolveDMXChannel() const
	{
		const TSharedPtr<FDMXGDTFMacroDMX> MacroDMX = OuterMacroDMXStep.IsValid() ? OuterMacroDMXStep.Pin()->OuterMacroDMX.Pin() : nullptr;
		const TSharedPtr<FDMXGDTFFTMacro> FTMacro = MacroDMX.IsValid() ? MacroDMX->OuterFTMacro.Pin() : nullptr;
		const TSharedPtr<FDMXGDTFDMXMode> DMXMode = FTMacro.IsValid() ? FTMacro->OuterDMXMode.Pin() : nullptr;
		if (DMXMode.IsValid())
		{
			TSharedPtr<class FDMXGDTFDMXChannel> ResolvedDMXChannel;
			TSharedPtr<FDMXGDTFChannelFunction> Dummy;
			DMXMode->ResolveChannel(DMXChannel, ResolvedDMXChannel, Dummy);

			return ResolvedDMXChannel;
		}

		return nullptr;
	}
}
