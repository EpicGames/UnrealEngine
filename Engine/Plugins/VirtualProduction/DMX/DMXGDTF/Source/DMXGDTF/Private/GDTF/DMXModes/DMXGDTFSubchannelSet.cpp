// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/DMXModes/DMXGDTFSubchannelSet.h"

#include "Algo/Find.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttribute.h"
#include "GDTF/AttributeDefinitions/DMXGDTFAttributeDefinitions.h"
#include "GDTF/AttributeDefinitions/DMXGDTFSubphysicalUnit.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFDMXProfile.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFPhysicalDescriptions.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFSubchannelSet::FDMXGDTFSubchannelSet(const TSharedRef<FDMXGDTFChannelFunction>& InChannelFunction)
		: OuterChannelFunction(InChannelFunction)
	{}

	void FDMXGDTFSubchannelSet::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("PhysicalFrom"), PhysicalFrom)
			.GetAttribute(TEXT("PhysicalTo"), PhysicalTo)
			.GetAttribute(TEXT("SubphyiscalUnit"), SubphyiscalUnit)
			.GetAttribute(TEXT("DMXProfile"), DMXProfile);
	}

	FXmlNode* FDMXGDTFSubchannelSet::CreateXmlNode(FXmlNode& Parent)
	{
		const float DefaultPhysicalFrom = 0.f;
		const float DefaultPhysicalTo = 1.f;

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("PhysicalFrom"), PhysicalFrom, DefaultPhysicalFrom)
			.SetAttribute(TEXT("PhysicalTo"), PhysicalTo, DefaultPhysicalTo)
			.SetAttribute(TEXT("SubphyiscalUnit"), SubphyiscalUnit)
			.SetAttribute(TEXT("DMXProfile"), DMXProfile);

		return ChildBuilder.GetIntermediateXmlNode();
	}
	TSharedPtr<FDMXGDTFSubphysicalUnit> FDMXGDTFSubchannelSet::ResolveSubphysicalUnit() const
	{
		TArray<FString> Link;
		SubphyiscalUnit.ParseIntoArray(Link, TEXT("."));
		if (Link.Num() != 2)
		{
			return nullptr;
		}

		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions = FixtureType.IsValid() ? FixtureType->AttributeDefinitions : nullptr;
		if (AttributeDefinitions.IsValid())
		{
			const TSharedPtr<FDMXGDTFAttribute>* AttributePtr = Algo::FindBy(AttributeDefinitions->Attributes, Link[0], &FDMXGDTFAttribute::Name);
			if (AttributePtr)
			{
				const UEnum* UEnumObject = StaticEnum<EDMXGDTFSubphysicalUnit>();
				int64 Index = UEnumObject->GetValueByName(*Link[1]);
				if (Index != INDEX_NONE)
				{
					const EDMXGDTFSubphysicalUnit SubphysicalUnitEnum = static_cast<EDMXGDTFSubphysicalUnit>(Index);
					const TSharedPtr<FDMXGDTFSubphysicalUnit>* SubphysicalUnitPtr = Algo::FindBy((*AttributePtr)->SubpyhsicalUnitArray, SubphysicalUnitEnum, &FDMXGDTFSubphysicalUnit::Type);

					if (SubphysicalUnitPtr)
					{
						return *SubphysicalUnitPtr;
					}
				}
			}
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFDMXProfile> FDMXGDTFSubchannelSet::ResolveDMXProfile() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			const TSharedPtr<FDMXGDTFDMXProfile>* DMXProfilePtr = Algo::FindBy(PhysicalDescriptions->DMXProfiles, DMXProfile, &FDMXGDTFDMXProfile::Name);
			if (DMXProfilePtr)
			{
				return *DMXProfilePtr;
			}
		}

		return nullptr;
	}
}
