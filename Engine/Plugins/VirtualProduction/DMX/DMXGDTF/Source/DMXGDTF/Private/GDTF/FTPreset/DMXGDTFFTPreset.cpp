// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/FTPresets/DMXGDTFFTPreset.h"

#include "GDTF/DMXGDTFFixtureType.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFFTPreset::FDMXGDTFFTPreset(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFFTPreset::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode);
	}

	FXmlNode* FDMXGDTFFTPreset::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
