// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFBeamGeometry.h"

#include "Algo/Find.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFEmitter.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFPhysicalDescriptions.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFBeamGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("LampType"), LampType)
			.GetAttribute(TEXT("PowerConsumption"), PowerConsumption)
			.GetAttribute(TEXT("LuminousFlux"), LuminousFlux)
			.GetAttribute(TEXT("ColorTemperature"), ColorTemperature)
			.GetAttribute(TEXT("BeamAngle"), BeamAngle)
			.GetAttribute(TEXT("FieldAngle"), FieldAngle)
			.GetAttribute(TEXT("ThrowRatio"), ThrowRatio)
			.GetAttribute(TEXT("RectangleRatio"), RectangleRatio)
			.GetAttribute(TEXT("BeamRadius"), BeamRadius)
			.GetAttribute(TEXT("BeamType"), BeamType)
			.GetAttribute(TEXT("ColorRenderingIndex"), ColorRenderingIndex)
			.GetAttribute(TEXT("EmitterSpectrum"), EmitterSpectrum);
	}

	FXmlNode* FDMXGDTFBeamGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		const float DefaultThrowRatio = 1.f;
		const float DefaultRecatangleRatio = 1.7777f;
		const FString DefaultEmitterSpectrum = TEXT("");

		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("LampType"), LampType)
			.SetAttribute(TEXT("PowerConsumption"), PowerConsumption)
			.SetAttribute(TEXT("LuminousFlux"), LuminousFlux)
			.SetAttribute(TEXT("ColorTemperature"), ColorTemperature)
			.SetAttribute(TEXT("BeamAngle"), BeamAngle)
			.SetAttribute(TEXT("FieldAngle"), FieldAngle)
			.SetAttribute(TEXT("ThrowRatio"), ThrowRatio, DefaultThrowRatio)
			.SetAttribute(TEXT("RectangleRatio"), RectangleRatio, DefaultRecatangleRatio)
			.SetAttribute(TEXT("BeamRadius"), BeamRadius)
			.SetAttribute(TEXT("BeamType"), BeamType)
			.SetAttribute(TEXT("ColorRenderingIndex"), ColorRenderingIndex)
			.SetAttribute(TEXT("EmitterSpectrum"), EmitterSpectrum, DefaultEmitterSpectrum);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFEmitter> FDMXGDTFBeamGeometry::ResolveEmitterSpectrum() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			if (const TSharedPtr<FDMXGDTFEmitter>* EmitterPtr = Algo::FindBy(PhysicalDescriptions->Emitters, EmitterSpectrum, &FDMXGDTFEmitter::Name))
			{
				return *EmitterPtr;
			}
		}
		return nullptr;
	}
}
