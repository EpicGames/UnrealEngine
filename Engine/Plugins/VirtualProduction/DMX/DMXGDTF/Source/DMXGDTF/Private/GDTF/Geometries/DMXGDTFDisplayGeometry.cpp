// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFDisplayGeometry.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFDisplayGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Texture"), Texture);
	}

	FXmlNode* FDMXGDTFDisplayGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("Texture"), Texture);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
