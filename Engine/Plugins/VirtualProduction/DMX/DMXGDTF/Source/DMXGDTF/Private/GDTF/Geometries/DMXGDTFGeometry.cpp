// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFGeometry.h"

#include "Algo/Find.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/Models/DMXGDTFModel.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFGeometry::FDMXGDTFGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
		: OuterGeometryCollect(InGeometryCollect)
	{}

	void FDMXGDTFGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometryCollectBase::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Model"), Model)
			.GetAttribute(TEXT("Position"), Position);
	}

	FXmlNode* FDMXGDTFGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		const FString DefaultModel = TEXT("");

		FXmlNode* AppendToNode = FDMXGDTFGeometryCollectBase::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Model"), Model, DefaultModel)
			.SetAttribute(TEXT("Position"), Position, EDMXGDTFMatrixType::Matrix4x4);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFGeometry> FDMXGDTFGeometry::FindGeometryByName(const TCHAR* InName) const
	{
		if (InName == Name)
		{
			// Allow to find mutable self
			FDMXGDTFGeometry* NonConstThis = const_cast<FDMXGDTFGeometry*>(this);
			return StaticCastSharedRef<FDMXGDTFGeometry>(NonConstThis->AsShared());
		}
		
		return FDMXGDTFGeometryCollectBase::FindGeometryByName(InName);
	}

	TSharedPtr<FDMXGDTFModel> FDMXGDTFGeometry::ResolveModel() const
	{
		if (const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin())
		{
			const TSharedPtr<FDMXGDTFModel>* ModelPtr = Algo::FindBy(FixtureType->Models, Model, &FDMXGDTFModel::Name);
			if (ModelPtr)
			{
				return *ModelPtr;
			}
		}

		return nullptr;
	}
}
