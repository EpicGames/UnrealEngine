// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFGeometryBreak.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFGeometryBreak::FDMXGDTFGeometryBreak(const TSharedRef<FDMXGDTFGeometryReference>& InGeometryReference)
		: OuterGeometryReference(InGeometryReference)
	{}

	void FDMXGDTFGeometryBreak::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("DMXOffset"), DMXOffset)
			.GetAttribute(TEXT("DMXBreak"), DMXBreak);
	}

	FXmlNode* FDMXGDTFGeometryBreak::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("DMXOffset"), DMXOffset)
			.SetAttribute(TEXT("DMXBreak"), DMXBreak);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
