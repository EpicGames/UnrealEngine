// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFGeometryCollect::FDMXGDTFGeometryCollect(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFGeometryCollect::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometryCollectBase::Initialize(XmlNode);
	}

	FXmlNode* FDMXGDTFGeometryCollect::CreateXmlNode(FXmlNode& Parent)
	{
		return FDMXGDTFGeometryCollectBase::CreateXmlNode(Parent);
	}
}
