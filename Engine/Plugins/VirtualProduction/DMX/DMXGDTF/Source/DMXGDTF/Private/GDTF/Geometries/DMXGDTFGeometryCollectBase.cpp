// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFGeometryCollectBase.h"

#include "GDTF/Geometries/DMXGDTFAxisGeometry.h"
#include "GDTF/Geometries/DMXGDTFBeamGeometry.h"
#include "GDTF/Geometries/DMXGDTFDisplayGeometry.h"
#include "GDTF/Geometries/DMXGDTFFilterBeamGeometry.h"
#include "GDTF/Geometries/DMXGDTFFilterColorGeometry.h"
#include "GDTF/Geometries/DMXGDTFFilterGoboGeometry.h"
#include "GDTF/Geometries/DMXGDTFFilterShaperGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryReference.h"
#include "GDTF/Geometries/DMXGDTFInventoryGeometry.h"
#include "GDTF/Geometries/DMXGDTFLaserGeometry.h"
#include "GDTF/Geometries/DMXGDTFMagnetGeometry.h"
#include "GDTF/Geometries/DMXGDTFMagnetGeometry.h"
#include "GDTF/Geometries/DMXGDTFMediaServerCameraGeometry.h"
#include "GDTF/Geometries/DMXGDTFMediaServerLayerGeometry.h"
#include "GDTF/Geometries/DMXGDTFMediaServerMasterGeometry.h"
#include "GDTF/Geometries/DMXGDTFStructureGeometry.h"
#include "GDTF/Geometries/DMXGDTFSupportGeometry.h"
#include "GDTF/Geometries/DMXGDTFWiringObjectGeometry.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	const TCHAR* FDMXGDTFGeometryCollectBase::GetXmlTag() const
	{
		ensureMsgf(0, TEXT("Unexpected call to FDMXGDTFGeometryCollectBase::GetXmlTag in abstract FDMXGDTFGeometryCollectBase."));
		return TEXT("Invalid");
	}

	void FDMXGDTFGeometryCollectBase::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateChildren(TEXT("Geometry"), GeometryArray)
			.CreateChildren(TEXT("Axis"), AxisArray)
			.CreateChildren(TEXT("FilterBeam"), FilterBeamArray)
			.CreateChildren(TEXT("FilterColor"), FilterColorArray)
			.CreateChildren(TEXT("FilterGobo"), FilterGoboArray)
			.CreateChildren(TEXT("FilterShaper"), FilterShaperArray)
			.CreateChildren(TEXT("Beam"), BeamArray)
			.CreateChildren(TEXT("MediaServerLayer"), MediaServerLayerArray)
			.CreateChildren(TEXT("MediaServerCamera"), MediaServerCameraArray)
			.CreateChildren(TEXT("MediaServerMaster"), MediaServerMasterArray)
			.CreateChildren(TEXT("Display"), DisplayArray)
			.CreateChildren(TEXT("GeometryReference"), GeometryReferenceArray)
			.CreateChildren(TEXT("Laser"), LaserArray)
			.CreateChildren(TEXT("WiringObject"), WiringObjectArray)
			.CreateChildren(TEXT("Inventory"), InventoryArray)
			.CreateChildren(TEXT("Structure"), StructureArray)
			.CreateChildren(TEXT("Support"), SupportArray)
			.CreateChildren(TEXT("Magnet"), MagnetArray);
	}

	FXmlNode* FDMXGDTFGeometryCollectBase::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendChildren(TEXT("Geometry"), GeometryArray)
			.AppendChildren(TEXT("Axis"), AxisArray)
			.AppendChildren(TEXT("FilterBeam"), FilterBeamArray)
			.AppendChildren(TEXT("FilterColor"), FilterColorArray)
			.AppendChildren(TEXT("FilterGobo"), FilterGoboArray)
			.AppendChildren(TEXT("FilterShaper"), FilterShaperArray)
			.AppendChildren(TEXT("Beam"), BeamArray)
			.AppendChildren(TEXT("MediaServerLayer"), MediaServerLayerArray)
			.AppendChildren(TEXT("MediaServerCamera"), MediaServerCameraArray)
			.AppendChildren(TEXT("MediaServerMaster"), MediaServerMasterArray)
			.AppendChildren(TEXT("Display"), DisplayArray)
			.AppendChildren(TEXT("GeometryReference"), GeometryReferenceArray)
			.AppendChildren(TEXT("Laser"), LaserArray)
			.AppendChildren(TEXT("WiringObject"), WiringObjectArray)
			.AppendChildren(TEXT("Inventory"), InventoryArray)
			.AppendChildren(TEXT("Structure"), StructureArray)
			.AppendChildren(TEXT("Support"), SupportArray)
			.AppendChildren(TEXT("Magnet"), MagnetArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	void FDMXGDTFGeometryCollectBase::GetGeometriesRecursive(TArray<TSharedPtr<FDMXGDTFGeometry>>& OutGeometries, TArray<TSharedPtr<FDMXGDTFGeometryReference>>& OutGeometryReferences)
	{
		for (const TSharedPtr<FDMXGDTFGeometryReference>& GeometryReference : GeometryReferenceArray)
		{
			OutGeometryReferences.Add(GeometryReference);
		}

		const auto GetChildrenRecursiveLambda = [this, &OutGeometries, &OutGeometryReferences](const auto& Geometries)
			{
				for (const TSharedPtr<FDMXGDTFGeometry>& Geometry : Geometries)
				{
					if (Geometry.IsValid())
					{
						OutGeometries.Add(Geometry);

						Geometry->GetGeometriesRecursive(OutGeometries, OutGeometryReferences);
					}
				}
			};

		// Recursive for children
		GetChildrenRecursiveLambda(GeometryArray);
		GetChildrenRecursiveLambda(AxisArray);
		GetChildrenRecursiveLambda(FilterBeamArray);
		GetChildrenRecursiveLambda(FilterColorArray);
		GetChildrenRecursiveLambda(FilterGoboArray);
		GetChildrenRecursiveLambda(FilterShaperArray);
		GetChildrenRecursiveLambda(BeamArray);
		GetChildrenRecursiveLambda(MediaServerLayerArray);
		GetChildrenRecursiveLambda(MediaServerCameraArray);
		GetChildrenRecursiveLambda(MediaServerMasterArray);
		GetChildrenRecursiveLambda(DisplayArray);
		GetChildrenRecursiveLambda(LaserArray);
		GetChildrenRecursiveLambda(WiringObjectArray);
		GetChildrenRecursiveLambda(InventoryArray);
		GetChildrenRecursiveLambda(StructureArray);
		GetChildrenRecursiveLambda(SupportArray);
		GetChildrenRecursiveLambda(MagnetArray);
	}

	TSharedPtr<FDMXGDTFGeometry> FDMXGDTFGeometryCollectBase::FindGeometryByName(const TCHAR* InName) const
	{
		// Helper to find geometry nodes in arrays of different types
		auto FindInArrayLambda = [InName](auto InArray, TSharedPtr<FDMXGDTFGeometry>& OutGeometry) -> bool
			{
				auto const* GeometryPtr = Algo::FindBy(InArray, InName, &FDMXGDTFGeometry::Name);
				OutGeometry = GeometryPtr ? *GeometryPtr : nullptr;
				return GeometryPtr != nullptr;
			};

		TSharedPtr<FDMXGDTFGeometry> Geometry;
		if (FindInArrayLambda(GeometryArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(AxisArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(FilterBeamArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(FilterColorArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(FilterGoboArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(FilterShaperArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(BeamArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(MediaServerLayerArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(MediaServerCameraArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(MediaServerMasterArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(DisplayArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(LaserArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(WiringObjectArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(InventoryArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(StructureArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(SupportArray, Geometry)) { return Geometry; }
		else if (FindInArrayLambda(MagnetArray, Geometry)) { return Geometry; }

		return nullptr;
	}

	TSharedPtr<FDMXGDTFGeometryReference> FDMXGDTFGeometryCollectBase::FindGeometryReferenceByName(const TCHAR* InName) const
	{
		const TSharedPtr<FDMXGDTFGeometryReference>* GeometryReferencePtr = Algo::FindBy(GeometryReferenceArray, InName, &FDMXGDTFGeometryReference::Name);

		return GeometryReferencePtr ? *GeometryReferencePtr : nullptr;
	}
}
