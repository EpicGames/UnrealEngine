// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFGeometryReference.h"

#include "Algo/Find.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryBreak.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Models/DMXGDTFModel.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFGeometryReference::FDMXGDTFGeometryReference(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
		: OuterGeometryCollect(InGeometryCollect)
	{}

	void FDMXGDTFGeometryReference::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Position"), Position)
			.CreateChildren(TEXT("Break"), BreakArray)
			.GetAttribute(TEXT("Geometry"), Geometry)
			.GetAttribute(TEXT("Model"), Model);
	}

	FXmlNode* FDMXGDTFGeometryReference::CreateXmlNode(FXmlNode& Parent)
	{
		const FName DefaultModel = "";

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Position"), Position, EDMXGDTFMatrixType::Matrix4x4)
			.AppendChildren(TEXT("Break"), BreakArray)
			.SetAttribute(TEXT("Geometry"), Geometry)
			.SetAttribute(TEXT("Model"), Model, DefaultModel);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFGeometry> FDMXGDTFGeometryReference::ResolveGeometry() const
	{
		// Only top level geometries are allowed to be referenced 
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFGeometryCollect> GeometryCollect = FixtureType.IsValid() ? FixtureType->GeometryCollect : nullptr;
		if (GeometryCollect.IsValid())
		{
			const TSharedPtr<FDMXGDTFGeometry>* GeometryPtr = Algo::FindBy(GeometryCollect->GeometryArray, Geometry, &FDMXGDTFGeometry::Name);
			if (GeometryPtr)
			{
				return *GeometryPtr;
			}
		}

		return nullptr;
	}

	TSharedPtr<FDMXGDTFModel> FDMXGDTFGeometryReference::ResolveModel() const
	{
		if (const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin())
		{
			const TSharedPtr<FDMXGDTFModel>* ModelPtr = Algo::FindBy(FixtureType->Models, Model, &FDMXGDTFModel::Name);
			if (ModelPtr)
			{
				return *ModelPtr;
			}
		}

		return nullptr;
	}
}
