// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFInventoryGeometry.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFInventoryGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Cout"), Count);
	}

	FXmlNode* FDMXGDTFInventoryGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("Cout"), Count);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
