// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFLaserGeometry.h"

#include "GDTF/Geometries/DMXGDTFLaserProtocol.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFEmitter.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFPhysicalDescriptions.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFLaserGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		using namespace UE::DMX::GDTF;

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("ColorType"), ColorType)
			.GetAttribute(TEXT("Color"), Color)
			.GetAttribute(TEXT("OutputStrength"), OutputStrength)
			.GetAttribute(TEXT("BeamDiameter"), BeamDiameter)
			.GetAttribute(TEXT("Emitter"), Emitter)
			.GetAttribute(TEXT("BeamDivergenceMin"), BeamDivergenceMin)
			.GetAttribute(TEXT("BeamDivergenceMax"), BeamDivergenceMax)
			.GetAttribute(TEXT("ScanAnglePan"), ScanAnglePan)
			.GetAttribute(TEXT("ScanAngleTilt"), ScanAngleTilt)
			.GetAttribute(TEXT("ScanSpeed"), ScanSpeed)
			.CreateChildren(TEXT("Protocol"), ProtocolArray);
	}

	FXmlNode* FDMXGDTFLaserGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("ColorType"), ColorType)
			.SetAttribute(TEXT("Color"), Color)
			.SetAttribute(TEXT("OutputStrength"), OutputStrength)
			.SetAttribute(TEXT("BeamDiameter"), BeamDiameter)
			.SetAttribute(TEXT("Emitter"), Emitter)
			.SetAttribute(TEXT("BeamDivergenceMin"), BeamDivergenceMin)
			.SetAttribute(TEXT("BeamDivergenceMax"), BeamDivergenceMax)
			.SetAttribute(TEXT("ScanAnglePan"), ScanAnglePan)
			.SetAttribute(TEXT("ScanAngleTilt"), ScanAngleTilt)
			.SetAttribute(TEXT("ScanSpeed"), ScanSpeed)
			.AppendChildren(TEXT("Protocol"), ProtocolArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFEmitter> FDMXGDTFLaserGeometry::ResolveEmitter() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions = FixtureType.IsValid() ? FixtureType->PhysicalDescriptions : nullptr;
		if (PhysicalDescriptions.IsValid())
		{
			if (const TSharedPtr<FDMXGDTFEmitter>* EmitterPtr = Algo::FindBy(PhysicalDescriptions->Emitters, Emitter, &FDMXGDTFEmitter::Name))
			{
				return *EmitterPtr;
			}
		}
		return nullptr;
	}
}
