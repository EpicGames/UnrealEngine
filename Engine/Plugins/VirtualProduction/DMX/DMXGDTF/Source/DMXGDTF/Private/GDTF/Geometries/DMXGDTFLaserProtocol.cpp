// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFLaserProtocol.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFLaserProtocol::FDMXGDTFLaserProtocol(const TSharedRef<FDMXGDTFLaserGeometry>& InLaserGeometry)
		: OuterLaserGeometry(InLaserGeometry)
	{}

	void FDMXGDTFLaserProtocol::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name);
	}
	
	FXmlNode* FDMXGDTFLaserProtocol::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
