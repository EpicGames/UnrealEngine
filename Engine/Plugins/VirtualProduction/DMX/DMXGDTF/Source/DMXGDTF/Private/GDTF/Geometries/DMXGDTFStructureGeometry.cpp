// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFStructureGeometry.h"

#include "Algo/Find.h"
#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFStructureGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("StructureType"), StructureType)
			.GetAttribute(TEXT("CrossSectionType"), CrossSectionType)
			.GetAttribute(TEXT("CrossSectionHeight"), CrossSectionHeight)
			.GetAttribute(TEXT("CrossSectionWallThickness"), CrossSectionWallThickness)
			.GetAttribute(TEXT("TrussCrossSection"), TrussCrossSection)
			.GetAttribute(TEXT("LinkedGeometry"), LinkedGeometry);
	}

	FXmlNode* FDMXGDTFStructureGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("StructureType"), StructureType)
			.SetAttribute(TEXT("CrossSectionType"), CrossSectionType)
			.SetAttribute(TEXT("CrossSectionHeight"), CrossSectionHeight)
			.SetAttribute(TEXT("CrossSectionWallThickness"), CrossSectionWallThickness)
			.SetAttribute(TEXT("TrussCrossSection"), TrussCrossSection)
			.SetAttribute(TEXT("LinkedGeometry"), LinkedGeometry);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFGeometry> FDMXGDTFStructureGeometry::ResolveLinkedGeometry() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFGeometryCollect> GeometryCollect = FixtureType.IsValid() ? FixtureType->GeometryCollect : nullptr;
		if (GeometryCollect.IsValid())
		{
			const TSharedPtr<FDMXGDTFGeometry>* LinkedGeometryPtr = Algo::FindBy(GeometryCollect->GeometryArray, LinkedGeometry, &FDMXGDTFGeometry::Name);
			if (LinkedGeometryPtr)
			{
				return *LinkedGeometryPtr;
			}
		}

		return nullptr;
	}
}
