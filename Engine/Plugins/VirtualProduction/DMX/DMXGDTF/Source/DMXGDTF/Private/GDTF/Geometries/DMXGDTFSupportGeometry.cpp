// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFSupportGeometry.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFSupportGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("SupportType"), SupportType)
			.GetAttribute(TEXT("RopeCrossSection"), RopeCrossSection)
			.GetAttribute(TEXT("RopeOffset"), RopeOffset)
			.GetAttribute(TEXT("CapacityX"), CapacityX)
			.GetAttribute(TEXT("CapacityY"), CapacityY)
			.GetAttribute(TEXT("CapacityZ"), CapacityZ)
			.GetAttribute(TEXT("CapacityXX"), CapacityXX)
			.GetAttribute(TEXT("CapacityYY"), CapacityYY)
			.GetAttribute(TEXT("CapacityZZ"), CapacityZZ)
			.GetAttribute(TEXT("ResistanceX"), ResistanceX)
			.GetAttribute(TEXT("ResistanceY"), ResistanceY)
			.GetAttribute(TEXT("ResistanceZ"), ResistanceZ)
			.GetAttribute(TEXT("ResistanceXX"), ResistanceXX)
			.GetAttribute(TEXT("ResistanceYY"), ResistanceYY)
			.GetAttribute(TEXT("ResistanceZZ"), ResistanceZZ);
	}

	FXmlNode* FDMXGDTFSupportGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("SupportType"), SupportType)
			.SetAttribute(TEXT("RopeCrossSection"), RopeCrossSection)
			.SetAttribute(TEXT("RopeOffset"), RopeOffset)
			.SetAttribute(TEXT("CapacityX"), CapacityX)
			.SetAttribute(TEXT("CapacityY"), CapacityY)
			.SetAttribute(TEXT("CapacityZ"), CapacityZ)
			.SetAttribute(TEXT("CapacityXX"), CapacityXX)
			.SetAttribute(TEXT("CapacityYY"), CapacityYY)
			.SetAttribute(TEXT("CapacityZZ"), CapacityZZ)
			.SetAttribute(TEXT("ResistanceX"), ResistanceX)
			.SetAttribute(TEXT("ResistanceY"), ResistanceY)
			.SetAttribute(TEXT("ResistanceZ"), ResistanceZ)
			.SetAttribute(TEXT("ResistanceXX"), ResistanceXX)
			.SetAttribute(TEXT("ResistanceYY"), ResistanceYY)
			.SetAttribute(TEXT("ResistanceZZ"), ResistanceZZ);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
