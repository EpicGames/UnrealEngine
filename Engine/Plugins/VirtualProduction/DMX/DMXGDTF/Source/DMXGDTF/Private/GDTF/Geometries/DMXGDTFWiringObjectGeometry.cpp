// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFWiringObjectGeometry.h"

#include "Algo/Find.h"
#include "GDTF/Geometries/DMXGDTFWiringObjectPinPatch.h"
#include "GDTF/Models/DMXGDTFModel.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFWiringObjectGeometry::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFGeometry::Initialize(XmlNode);

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("ConnectorType"), ConnectorType)
			.GetAttribute(TEXT("Matrix"), Matrix)
			.GetAttribute(TEXT("ComponentType"), ComponentType)
			.GetAttribute(TEXT("SignalType"), SignalType)
			.GetAttribute(TEXT("PinCount"), PinCount)
			.GetAttribute(TEXT("ElectricalPayLoad"), ElectricalPayLoad)
			.GetAttribute(TEXT("VoltageRangeMax"), VoltageRangeMax)
			.GetAttribute(TEXT("VoltageRangeMin"), VoltageRangeMin)
			.GetAttribute(TEXT("FrequencyRangeMax"), FrequencyRangeMax)
			.GetAttribute(TEXT("FrequencyRangeMin"), FrequencyRangeMin)
			.GetAttribute(TEXT("MaxPayLoad"), MaxPayLoad)
			.GetAttribute(TEXT("Voltage"), Voltage)
			.GetAttribute(TEXT("SignalLayer"), SignalLayer)
			.GetAttribute(TEXT("CosPhi"), CosPhi)
			.GetAttribute(TEXT("FuseCurrent"), FuseCurrent)
			.GetAttribute(TEXT("FuseRating"), FuseRating)
			.GetAttribute(TEXT("Orientation"), Orientation)
			.GetAttribute(TEXT("WireGroup"), WireGroup)
			.CreateChildren(TEXT("PinPatch"), PinPatchArray);
	}

	FXmlNode* FDMXGDTFWiringObjectGeometry::CreateXmlNode(FXmlNode& Parent)
	{
		const FTransform DefaultMatrix = FTransform::Identity;

		FXmlNode* AppendToNode = FDMXGDTFGeometry::CreateXmlNode(Parent);

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this, AppendToNode)
			.SetAttribute(TEXT("ConnectorType"), ConnectorType)
			.SetAttribute(TEXT("Matrix"), Matrix, EDMXGDTFMatrixType::Matrix4x4, DefaultMatrix)
			.SetAttribute(TEXT("ComponentType"), ComponentType)
			.SetAttribute(TEXT("SignalType"), SignalType)
			.SetAttribute(TEXT("PinCount"), PinCount)
			.SetAttribute(TEXT("ElectricalPayLoad"), ElectricalPayLoad)
			.SetAttribute(TEXT("VoltageRangeMax"), VoltageRangeMax)
			.SetAttribute(TEXT("VoltageRangeMin"), VoltageRangeMin)
			.SetAttribute(TEXT("FrequencyRangeMax"), FrequencyRangeMax)
			.SetAttribute(TEXT("FrequencyRangeMin"), FrequencyRangeMin)
			.SetAttribute(TEXT("MaxPayLoad"), MaxPayLoad)
			.SetAttribute(TEXT("Voltage"), Voltage)
			.SetAttribute(TEXT("SignalLayer"), SignalLayer)
			.SetAttribute(TEXT("CosPhi"), CosPhi)
			.SetAttribute(TEXT("FuseCurrent"), FuseCurrent)
			.SetAttribute(TEXT("FuseRating"), FuseRating)
			.SetAttribute(TEXT("Orientation"), Orientation)
			.SetAttribute(TEXT("WireGroup"), WireGroup)
			.AppendChildren(TEXT("PinPatch"), PinPatchArray);
		
		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFModel> FDMXGDTFWiringObjectGeometry::ResolveModel() const
	{
		if (const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin())
		{
			const TSharedPtr<FDMXGDTFModel>* ModelPtr = Algo::FindBy(FixtureType->Models, Model, &FDMXGDTFModel::Name);
			if (ModelPtr)
			{
				return *ModelPtr;
			}
		}

		return nullptr;
	}
}
