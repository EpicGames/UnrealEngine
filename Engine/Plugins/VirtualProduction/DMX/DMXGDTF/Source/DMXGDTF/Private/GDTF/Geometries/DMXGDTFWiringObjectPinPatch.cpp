// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Geometries/DMXGDTFWiringObjectPinPatch.h"

#include "GDTF/Geometries/DMXGDTFGeometryCollect.h"
#include "GDTF/Geometries/DMXGDTFWiringObjectGeometry.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFWiringObjectPinPatch::FDMXGDTFWiringObjectPinPatch(const TSharedRef<FDMXGDTFWiringObjectGeometry>& InWiringObjectGeometry)
		: OuterWiringObjectGeometry(InWiringObjectGeometry)
	{}

	void FDMXGDTFWiringObjectPinPatch::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("FromPin"), FromPin)
			.GetAttribute(TEXT("ToPin"), ToPin)
			.GetAttribute(TEXT("ToWiringObject"), ToWiringObject);
	}

	FXmlNode* FDMXGDTFWiringObjectPinPatch::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("FromPin"), FromPin)
			.SetAttribute(TEXT("ToPin"), ToPin)
			.SetAttribute(TEXT("ToWiringObject"), ToWiringObject);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFWiringObjectGeometry> FDMXGDTFWiringObjectPinPatch::ResolveToWiringObject() const
	{
		const TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin();
		const TSharedPtr<FDMXGDTFGeometryCollectBase> GeometryCollect = FixtureType.IsValid() ? FixtureType->GeometryCollect : nullptr;
		if (GeometryCollect.IsValid())
		{
			constexpr bool bRecursive = true;
			return GeometryCollect->ResolveGeometryLink<FDMXGDTFWiringObjectGeometry>(ToWiringObject);
		}

		return nullptr;
	}
}
