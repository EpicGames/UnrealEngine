// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Models/DMXGDTFModel.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFModel::FDMXGDTFModel(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFModel::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Length"), Length)
			.GetAttribute(TEXT("Width"), Width)
			.GetAttribute(TEXT("Height"), Height)
			.GetAttribute(TEXT("PrimitiveType"), PrimitiveType)
			.GetAttribute(TEXT("File"), File)
			.GetAttribute(TEXT("SVGOffsetX"), SVGOffsetX)
			.GetAttribute(TEXT("SVGOffsetY"), SVGOffsetY)
			.GetAttribute(TEXT("SVGSideOffsetX"), SVGSideOffsetX)
			.GetAttribute(TEXT("SVGSideOffsetY"), SVGSideOffsetY)
			.GetAttribute(TEXT("SVGFrontOffsetX"), SVGFrontOffsetX)
			.GetAttribute(TEXT("SVGFrontOffsetY"), SVGFrontOffsetY);
	}

	FXmlNode* FDMXGDTFModel::CreateXmlNode(FXmlNode& Parent)
	{
		const float DefaultSVGOffsetValue = 0.f;

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Length"), Length)
			.SetAttribute(TEXT("Width"), Width)
			.SetAttribute(TEXT("Height"), Height)
			.SetAttribute(TEXT("PrimitiveType"), PrimitiveType)
			.SetAttribute(TEXT("File"), File)
			.SetAttribute(TEXT("SVGOffsetX"), SVGOffsetX, DefaultSVGOffsetValue)
			.SetAttribute(TEXT("SVGOffsetY"), SVGOffsetY, DefaultSVGOffsetValue)
			.SetAttribute(TEXT("SVGSideOffsetX"), SVGSideOffsetX, DefaultSVGOffsetValue)
			.SetAttribute(TEXT("SVGSideOffsetY"), SVGSideOffsetY, DefaultSVGOffsetValue)
			.SetAttribute(TEXT("SVGFrontOffsetX"), SVGFrontOffsetX, DefaultSVGOffsetValue)
			.SetAttribute(TEXT("SVGFrontOffsetY"), SVGFrontOffsetY, DefaultSVGOffsetValue);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
