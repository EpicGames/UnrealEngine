// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFColorRenderingIndex.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFColorRenderingIndex::FDMXGDTFColorRenderingIndex(const TSharedRef<FDMXGDTFColorRenderingIndexGroup>& InColorRenderingIndexGroup)
		: OuterColorRenderingIndexGroup(InColorRenderingIndexGroup)
	{}

	void FDMXGDTFColorRenderingIndex::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("CES"), CES, this, &FDMXGDTFColorRenderingIndex::ParseCES)
			.GetAttribute(TEXT("ColorRenderingIndex"), ColorRenderingIndex);
	}

	FXmlNode* FDMXGDTFColorRenderingIndex::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("CES"), CES)
			.SetAttribute(TEXT("ColorRenderingIndex"), ColorRenderingIndex);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	uint8 FDMXGDTFColorRenderingIndex::ParseCES(const FString& GDTFString) const
	{
		const FString CleanString = GDTFString.Replace(TEXT("CES"), TEXT(""));

		uint8 Result;
		if (LexTryParseString(Result, *CleanString))
		{
			return Result;
		}
		else
		{
			// Return the default
			return 1;
		}
	}
}
