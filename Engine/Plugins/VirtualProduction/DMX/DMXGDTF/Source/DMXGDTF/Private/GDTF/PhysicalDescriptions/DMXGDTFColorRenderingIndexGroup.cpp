// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFColorRenderingIndexGroup.h"

#include "GDTF/PhysicalDescriptions/DMXGDTFColorRenderingIndex.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFColorRenderingIndexGroup::FDMXGDTFColorRenderingIndexGroup(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions)
		: OuterPhysicalDescriptions(InPhysicalDescriptions)
	{}

	void FDMXGDTFColorRenderingIndexGroup::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("ColorTemperature"), ColorTemperature)
			.CreateChildren(TEXT("CRI"), CRIArray);
	}

	FXmlNode* FDMXGDTFColorRenderingIndexGroup::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("ColorTemperature"), ColorTemperature)
			.AppendChildren(TEXT("CRI"), CRIArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
