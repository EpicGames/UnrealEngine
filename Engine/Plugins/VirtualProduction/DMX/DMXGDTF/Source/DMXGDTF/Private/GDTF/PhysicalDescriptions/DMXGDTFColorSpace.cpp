// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFColorSpace.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFColorSpace::FDMXGDTFColorSpace(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions)
		: OuterPhysicalDescriptions(InPhysicalDescriptions)
	{}

	void FDMXGDTFColorSpace::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Mode"), Mode)
			.GetAttribute(TEXT("Red"), Red)
			.GetAttribute(TEXT("Green"), Green)
			.GetAttribute(TEXT("Blue"), Blue)
			.GetAttribute(TEXT("WhitePoint"), WhitePoint);
	}

	FXmlNode* FDMXGDTFColorSpace::CreateXmlNode(FXmlNode& Parent)
	{
		const FName DefaultName = "Default";
		const FDMXGDTFColorCIE1931xyY DefaultColor = { 0.f, 0.f, 0.f };

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name, DefaultName)
			.SetAttribute(TEXT("Mode"), Mode)
			.SetAttribute(TEXT("Red"), Red, DefaultColor)
			.SetAttribute(TEXT("Green"), Green, DefaultColor)
			.SetAttribute(TEXT("Blue"), Blue, DefaultColor)
			.SetAttribute(TEXT("WhitePoint"), WhitePoint, DefaultColor);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
