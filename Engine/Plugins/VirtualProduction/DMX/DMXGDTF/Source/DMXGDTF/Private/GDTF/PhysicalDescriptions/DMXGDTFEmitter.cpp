// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFEmitter.h"

#include "GDTF/PhysicalDescriptions/DMXGDTFMeasurement.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFEmitter::FDMXGDTFEmitter(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions)
		: OuterPhysicalDescriptions(InPhysicalDescriptions)
	{}

	void FDMXGDTFEmitter::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Color"), Color)
			.GetAttribute(TEXT("DominantWaveLength"), DominantWaveLength)
			.GetAttribute(TEXT("DiodePart"), DiodePart)
			.CreateChildren(TEXT("Measurement"), Measurements);
	}

	FXmlNode* FDMXGDTFEmitter::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Color"), Color)
			.SetAttribute(TEXT("DominantWaveLength"), DominantWaveLength)
			.SetAttribute(TEXT("DiodePart"), DiodePart)
			.AppendChildren(TEXT("Measurement"), Measurements);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
