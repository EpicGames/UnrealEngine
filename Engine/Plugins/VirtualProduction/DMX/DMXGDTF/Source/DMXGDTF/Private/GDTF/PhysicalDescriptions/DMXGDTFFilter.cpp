// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFFilter.h"

#include "GDTF/PhysicalDescriptions/DMXGDTFMeasurement.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFFilter::FDMXGDTFFilter(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions)
		: OuterPhysicalDescriptions(InPhysicalDescriptions)
	{}

	void FDMXGDTFFilter::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Color"), Color)
			.CreateChildren(TEXT("Measurement"), Measurements);
	}

	FXmlNode* FDMXGDTFFilter::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Color"), Color)
			.AppendChildren(TEXT("Measurement"), Measurements);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
