// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFGamut.h"

#include "DMXGDTFLog.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFGamut::FDMXGDTFGamut(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions)
		: OuterPhysicalDescriptions(InPhysicalDescriptions)
	{}

	void FDMXGDTFGamut::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Points"), Points, this, &FDMXGDTFGamut::ParsePoints);
	}

	FXmlNode* FDMXGDTFGamut::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Points"), Points);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TArray<FDMXGDTFColorCIE1931xyY> FDMXGDTFGamut::ParsePoints(const FString& GDTFString) const
	{
		const FString CleanString = GDTFString.Replace(TEXT("}"), TEXT(""));

		TArray<FString> Strings;
		GDTFString.ParseIntoArray(Strings, TEXT("{"));

		TArray<FDMXGDTFColorCIE1931xyY> Result;
		for (const FString& String : Strings)
		{
			TArray<FString> ComponentStrings;
			String.ParseIntoArray(ComponentStrings, TEXT(","));

			if (ComponentStrings.Num() != 3)
			{
				Result.Reset();

				UE_LOGF(LogDMXGDTF, Warning, "Failed to parse gamut points for %ls. Failed to find three components for vector.", *Name.ToString());
				break;
			}

			FDMXGDTFColorCIE1931xyY Point;
			if (LexTryParseString(Point.X, *ComponentStrings[0]) &&
				LexTryParseString(Point.Y, *ComponentStrings[1]) &&
				LexTryParseString(Point.YY, *ComponentStrings[2]))
			{
				Result.Add(Point);
			}
			else
			{
				Result.Reset();

				UE_LOGF(LogDMXGDTF, Warning, "Failed to parse gamut points for %ls. Failed to convert string to numeric value.", *Name.ToString());
				break;
			}
		}

		return Result;
	}
}
