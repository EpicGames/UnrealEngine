// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFLegHeight.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFLegHeight::FDMXGDTFLegHeight(const TSharedRef<FDMXGDTFProperties>& InProperties)
		: OuterProperties(InProperties)
	{}

	void FDMXGDTFLegHeight::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Value"), Value);
	}

	FXmlNode* FDMXGDTFLegHeight::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Value"), Value);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
