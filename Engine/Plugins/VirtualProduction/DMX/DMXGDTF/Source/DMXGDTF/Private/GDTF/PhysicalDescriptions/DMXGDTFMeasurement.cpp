// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFMeasurement.h"

#include "GDTF/PhysicalDescriptions/DMXGDTFMeasurementPoint.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFMeasurementBase::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Physical"), Physical)
			.GetAttribute(TEXT("LuminousIntensity"), LuminousIntensity)
			.GetAttribute(TEXT("Transmission"), Transmission)
			.GetAttribute(TEXT("InterpolationTo"), InterpolationTo)
			.CreateChildren(TEXT("MeasurementPoint"), MeasurementPointArray);
	}

	FXmlNode* FDMXGDTFMeasurementBase::CreateXmlNode(FXmlNode& Parent)
	{
		const float DefaultPhysical = 0.f;
		const float DefaultLuminousIntensity = 0.f;
		const float DefaultTransmission = 0.f;

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Physical"), Physical, DefaultPhysical)
			.SetAttribute(TEXT("LuminousIntensity"), LuminousIntensity, DefaultLuminousIntensity)
			.SetAttribute(TEXT("Transmission"), Transmission, DefaultTransmission)
			.SetAttribute(TEXT("InterpolationTo"), InterpolationTo)
			.AppendChildren(TEXT("MeasurementPoint"), MeasurementPointArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	FDMXGDTFEmitterMeasurement::FDMXGDTFEmitterMeasurement(const TSharedRef<FDMXGDTFEmitter>& InEmitter)
		: OuterEmitter(InEmitter)
	{}

	FDMXGDTFFilterMeasurement::FDMXGDTFFilterMeasurement(const TSharedRef<FDMXGDTFFilter>& InFilter)
		: OuterFilter(InFilter)
	{}
}
