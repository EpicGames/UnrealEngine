// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFMeasurementPoint.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFMeasurementPoint::FDMXGDTFMeasurementPoint(const TSharedRef<FDMXGDTFMeasurementBase>& InMeasurement)
		: OuterMeasurement(InMeasurement)
	{}

	void FDMXGDTFMeasurementPoint::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("WaveLength"), WaveLength)
			.GetAttribute(TEXT("Energy"), Energy);
	}

	FXmlNode* FDMXGDTFMeasurementPoint::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("WaveLength"), WaveLength)
			.SetAttribute(TEXT("Energy"), Energy);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
