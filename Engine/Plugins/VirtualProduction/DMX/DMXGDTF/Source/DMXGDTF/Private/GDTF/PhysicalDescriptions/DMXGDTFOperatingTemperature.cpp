// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFOperatingTemperature.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFOperatingTemperature::FDMXGDTFOperatingTemperature(const TSharedRef<FDMXGDTFProperties>& InProperties)
		: OuterProperties(InProperties)
	{}

	void FDMXGDTFOperatingTemperature::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Low"), Low)
			.GetAttribute(TEXT("High"), High);
	}

	FXmlNode* FDMXGDTFOperatingTemperature::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Low"), Low)
			.SetAttribute(TEXT("High"), High);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
