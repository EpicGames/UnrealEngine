// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFPhysicalDescriptions.h"

#include "GDTF/PhysicalDescriptions/DMXGDTFColorRenderingIndexGroup.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFColorSpace.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFDMXProfile.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFEmitter.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFFilter.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFGamut.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFProperties.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFPhysicalDescriptions::FDMXGDTFPhysicalDescriptions(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFPhysicalDescriptions::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateChildCollection(TEXT("Emitters"), TEXT("Emitter"), Emitters)
			.CreateChildCollection(TEXT("Filters"), TEXT("Filter"), Filters)
			.CreateOptionalChild(TEXT("ColorSpace"), ColorSpaces)
			.CreateChildCollection(TEXT("AdditionalColorSpaces"), TEXT("ColorSpace"), AdditionalColorSpaces)
			.CreateChildCollection(TEXT("Gamuts"), TEXT("Gamut"), Gamuts)
			.CreateChildCollection(TEXT("DMXProfiles"), TEXT("DMXProfile"), DMXProfiles)
			.CreateChildCollection(TEXT("CRIs"), TEXT("CRIGroup"), CRIs)
			.CreateOptionalChild(TEXT("Properties"), Properties);
	}

	FXmlNode* FDMXGDTFPhysicalDescriptions::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendChildCollection(TEXT("Emitters"), TEXT("Emitter"), Emitters)
			.AppendChildCollection(TEXT("Filters"), TEXT("Filter"), Filters)
			.AppendOptionalChild(TEXT("ColorSpace"), ColorSpaces)
			.AppendChildCollection(TEXT("AdditionalColorSpaces"), TEXT("AdditionalColorSpace"), AdditionalColorSpaces)
			.AppendChildCollection(TEXT("Gamuts"), TEXT("Gamut"), Gamuts)
			.AppendChildCollection(TEXT("DMXProfiles"), TEXT("DMXProfile"), DMXProfiles)
			.AppendChildCollection(TEXT("CRIs"), TEXT("CRIGroup"), CRIs)
			.AppendOptionalChild(TEXT("Properties"), Properties);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
