// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFDMXProfilePoint.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFDMXProfilePoint::FDMXGDTFDMXProfilePoint(const TSharedRef<FDMXGDTFDMXProfile>& InDMXProfile)
		: OuterDMXProfile(InDMXProfile)
	{}

	void FDMXGDTFDMXProfilePoint::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("DMXPercentage"), DMXPercentage)
			.GetAttribute(TEXT("CFC0"), CFC0)
			.GetAttribute(TEXT("CFC1"), CFC1)
			.GetAttribute(TEXT("CFC2"), CFC2)
			.GetAttribute(TEXT("CFC3"), CFC3);
	}

	FXmlNode* FDMXGDTFDMXProfilePoint::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("DMXPercentage"), DMXPercentage)
			.SetAttribute(TEXT("CFC0"), CFC0)
			.SetAttribute(TEXT("CFC1"), CFC1)
			.SetAttribute(TEXT("CFC2"), CFC2)
			.SetAttribute(TEXT("CFC3"), CFC3);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
