// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/PhysicalDescriptions/DMXGDTFProperties.h"

#include "GDTF/PhysicalDescriptions/DMXGDTFLegHeight.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFOperatingTemperature.h"
#include "GDTF/PhysicalDescriptions/DMXGDTFWeight.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFProperties::FDMXGDTFProperties(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions)
		: OuterPhysicalDescriptions(InPhysicalDescriptions)
	{}

	void FDMXGDTFProperties::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateOptionalChild(TEXT("OperatingTemperature"), OperatingTemperature)
			.CreateOptionalChild(TEXT("Weight"), Weight)
			.CreateOptionalChild(TEXT("LegHeight"), LegHeight);
	}

	FXmlNode* FDMXGDTFProperties::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendOptionalChild(TEXT("OperatingTemperature"), OperatingTemperature)
			.AppendOptionalChild(TEXT("Weight"), Weight)
			.AppendOptionalChild(TEXT("LegHeight"), LegHeight);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
