// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/ArtNet/DMXGDTFProtocolArtNet.h"

#include "GDTF/Protocols/DMXGDTFProtocolDMXMap.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF::ArtNet
{
	FDMXGDTFProtocolArtNet::FDMXGDTFProtocolArtNet(const TSharedRef<FDMXGDTFProtocols>& InProtocols)
		: OuterProtocols(InProtocols)
	{}

	void FDMXGDTFProtocolArtNet::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateChildren(TEXT("Map"), Maps);
	}

	FXmlNode* FDMXGDTFProtocolArtNet::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendChildren(TEXT("Map"), Maps);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
