// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/DMXGDTFProtocolDMXMap.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	void FDMXGDTFProtocolDMXMapBase::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Key"), Key)
			.GetAttribute(TEXT("Value"), Value);
	}

	FXmlNode* FDMXGDTFProtocolDMXMapBase::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Key"), Key)
			.SetAttribute(TEXT("Value"), Value);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	namespace ArtNet
	{
		FDMXGDTFProtocolArtNetDMXMap::FDMXGDTFProtocolArtNetDMXMap(const TSharedRef<ArtNet::FDMXGDTFProtocolArtNet>& InProtocolArtNet)
			: OuterProtocolArtNet(InProtocolArtNet)
		{}
	}

	namespace SACN
	{
		FDMXGDTFProtocolSACNDMXMap::FDMXGDTFProtocolSACNDMXMap(const TSharedRef<FDMXGDTFProtocolSACN>& InProtocolSACN)
			: OuterProtocolSACN(InProtocolSACN)
		{}
	}
}
