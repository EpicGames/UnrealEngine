// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/DMXGDTFProtocols.h"

#include "GDTF/Protocols/ArtNet/DMXGDTFProtocolArtNet.h"
#include "GDTF/Protocols/RDM/DMXGDTFProtocolFTRDM.h"
#include "GDTF/Protocols/sACN/DMXGDTFProtocolSACN.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFProtocols::FDMXGDTFProtocols(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFProtocols::Initialize(const FXmlNode& XmlNode)
	{
		using namespace RDM;

		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateOptionalChild(TEXT("FTRDM"), RDM)
			.CreateOptionalChild(TEXT("ArtNet"), ArtNet)
			.CreateOptionalChild(TEXT("sACN"), sACN);
	}

	FXmlNode* FDMXGDTFProtocols::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendOptionalChild(TEXT("FTRDM"), RDM)
			.AppendOptionalChild(TEXT("ArtNet"), ArtNet)
			.AppendOptionalChild(TEXT("sACN"), sACN);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
