// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/RDM/DMXGDTFDMXPersonality.h"

#include "Algo/Find.h"
#include "GDTF/DMXModes/DMXGDTFDMXMode.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF::RDM
{
	FDMXGDTFDMXPersonality::FDMXGDTFDMXPersonality(const TSharedRef<FDMXGDTFSoftwareVersionID>& InSoftwareVersionID)
		: OuterSoftwareVersionID(InSoftwareVersionID)
	{}

	void FDMXGDTFDMXPersonality::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Value"), Value, &FParse::HexNumber)
			.GetAttribute(TEXT("DMXMode"), DMXMode);
	}

	FXmlNode* FDMXGDTFDMXPersonality::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Value"), Value)
			.SetAttribute(TEXT("DMXMode"), DMXMode);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	TSharedPtr<FDMXGDTFDMXMode> FDMXGDTFDMXPersonality::ResolveDMXMode() const
	{
		if (TSharedPtr<FDMXGDTFFixtureType> FixtureType = GetFixtureType().Pin())
		{
			if (const TSharedPtr<FDMXGDTFDMXMode>* DMXModePtr = Algo::FindBy(FixtureType->DMXModes, DMXMode, &FDMXGDTFDMXMode::Name))
			{
				return *DMXModePtr;
			}
		}

		return nullptr;
	}
}
