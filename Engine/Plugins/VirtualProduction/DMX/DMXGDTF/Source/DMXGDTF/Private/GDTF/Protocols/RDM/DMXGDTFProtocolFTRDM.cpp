// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/RDM/DMXGDTFProtocolFTRDM.h"

#include "GDTF/Protocols/RDM/DMXGDTFSoftwareVersionID.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF::RDM
{
	FDMXGDTFProtocolFTRDM::FDMXGDTFProtocolFTRDM(const TSharedRef<FDMXGDTFProtocols>& InProtocols)
		: OuterProtocols(InProtocols)
	{}

	void FDMXGDTFProtocolFTRDM::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("ManufacturerID"), ManufacturerID, &FParse::HexNumber)
			.GetAttribute(TEXT("DeviceModelID"), DeviceModelID, &FParse::HexNumber)
			.CreateChildren(TEXT("SoftwareVersionID"), SoftwareVersionIDArray);
	}

	FXmlNode* FDMXGDTFProtocolFTRDM::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("ManufacturerID"), ManufacturerID)
			.SetAttribute(TEXT("DeviceModelID"), DeviceModelID)
			.AppendChildren(TEXT("SoftwareVersionID"), SoftwareVersionIDArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
