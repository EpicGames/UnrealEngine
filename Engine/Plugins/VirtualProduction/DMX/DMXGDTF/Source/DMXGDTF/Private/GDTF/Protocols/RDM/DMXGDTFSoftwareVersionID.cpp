// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/RDM/DMXGDTFSoftwareVersionID.h"

#include "GDTF/Protocols/RDM/DMXGDTFDMXPersonality.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF::RDM
{
	FDMXGDTFSoftwareVersionID::FDMXGDTFSoftwareVersionID(const TSharedRef<FDMXGDTFProtocolFTRDM>& InProtocolRDM)
		: OuterProtocolRDM(InProtocolRDM)
	{}

	void FDMXGDTFSoftwareVersionID::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Value"), Value, &FParse::HexNumber)
			.CreateChildren(TEXT("DMXPersonality"), DMXPersonalityArray);
	}

	FXmlNode* FDMXGDTFSoftwareVersionID::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Value"), Value)
			.AppendChildren(TEXT("DMXPersonality"), DMXPersonalityArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
