// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Protocols/sACN/DMXGDTFProtocolSACN.h"

#include "GDTF/Protocols/DMXGDTFProtocolDMXMap.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF::SACN
{
	FDMXGDTFProtocolSACN::FDMXGDTFProtocolSACN(const TSharedRef<FDMXGDTFProtocols>& InProtocols)
		: OuterProtocols(InProtocols)
	{}

	void FDMXGDTFProtocolSACN::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.CreateChildren(TEXT("Map"), Maps);
	}

	FXmlNode* FDMXGDTFProtocolSACN::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.AppendChildren(TEXT("Map"), Maps);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
