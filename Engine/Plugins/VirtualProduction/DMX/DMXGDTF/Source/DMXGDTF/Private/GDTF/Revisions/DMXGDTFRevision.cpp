// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Revisions/DMXGDTFRevision.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFRevision::FDMXGDTFRevision(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFRevision::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Text"), Text)
			.GetAttribute(TEXT("Date"), Date, this, &FDMXGDTFRevision::ParseDateTime)
			.GetAttribute(TEXT("UserID"), UserID)
			.GetAttribute(TEXT("ModifiedBy"), ModifiedBy);
	}

	FXmlNode* FDMXGDTFRevision::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Text"), Text)
			.SetAttribute(TEXT("Date"), Date)
			.SetAttribute(TEXT("UserID"), UserID)
			.SetAttribute(TEXT("ModifiedBy"), ModifiedBy);

		return ChildBuilder.GetIntermediateXmlNode();
	}

	FDateTime FDMXGDTFRevision::ParseDateTime(const FString& GDTFString) const
	{
		const FRegexPattern DateTimePattern(TEXT("(\\d{4})-(\\d{2})-(\\d{2})T(\\d{2}):(\\d{2}):(\\d{2})"));
		FRegexMatcher Regex(DateTimePattern, *GDTFString);
		if (Regex.FindNext())
		{
			const FString YearString = Regex.GetCaptureGroup(1);
			const FString MonthString = Regex.GetCaptureGroup(2);
			const FString DayString = Regex.GetCaptureGroup(3);
			const FString HourString = Regex.GetCaptureGroup(4);
			const FString MinuteString = Regex.GetCaptureGroup(5);
			const FString SecondString = Regex.GetCaptureGroup(6);

			if (!YearString.IsEmpty() || !MonthString.IsEmpty() || !DayString.IsEmpty() || !HourString.IsEmpty() || !MinuteString.IsEmpty() || !SecondString.IsEmpty())
			{
				int32 Year;
				int32 Month;
				int32 Day;
				int32 Hour;
				int32 Minute;
				int32 Second;
				constexpr int32 Millisecond = 0;
				if (LexTryParseString(Year, *YearString) &&
					LexTryParseString(Month, *MonthString) &&
					LexTryParseString(Day, *DayString) &&
					LexTryParseString(Hour, *HourString) &&
					LexTryParseString(Minute, *MinuteString) &&
					LexTryParseString(Second, *SecondString) &&
					FDateTime::Validate(Year, Month, Day, Hour, Minute, Second, Millisecond))
				{
					return FDateTime(Year, Month, Day, Hour, Minute, Second, Millisecond);
				}
			}
		}
		
		return FDateTime::MinValue();
	}
}
