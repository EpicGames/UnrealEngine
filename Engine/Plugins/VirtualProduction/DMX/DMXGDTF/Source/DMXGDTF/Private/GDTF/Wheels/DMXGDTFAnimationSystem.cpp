// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Wheels/DMXGDTFAnimationSystem.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFAnimationSystem::FDMXGDTFAnimationSystem(const TSharedRef<FDMXGDTFWheelSlot>& InWheelSlot)
		: OuterWheelSlot(InWheelSlot)
	{}

	void FDMXGDTFAnimationSystem::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("P1"), P1)
			.GetAttribute(TEXT("P2"), P2)
			.GetAttribute(TEXT("P3"), P3)
			.GetAttribute(TEXT("Radius"), Radius);
	}

	FXmlNode* FDMXGDTFAnimationSystem::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("P1"), P1)
			.SetAttribute(TEXT("P2"), P2)
			.SetAttribute(TEXT("P3"), P3)
			.SetAttribute(TEXT("Radius"), Radius);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
