// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Wheels/DMXGDTFPrismFacet.h"

#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFPrismFacet::FDMXGDTFPrismFacet(const TSharedRef<FDMXGDTFWheelSlot>& InWheelSlot)
		: OuterWheelSlot(InWheelSlot)
	{}

	void FDMXGDTFPrismFacet::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Color"), Color)
			.GetAttribute(TEXT("Rotation"), Rotation);
	}

	FXmlNode* FDMXGDTFPrismFacet::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Color"), Color)
			.SetAttribute(TEXT("Rotation"), Rotation, EDMXGDTFMatrixType::Matrix3x3);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
