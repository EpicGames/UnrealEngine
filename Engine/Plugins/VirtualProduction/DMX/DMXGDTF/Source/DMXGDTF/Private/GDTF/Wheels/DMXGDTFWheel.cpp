// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Wheels/DMXGDTFWheel.h"

#include "GDTF/Wheels/DMXGDTFWheelSlot.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFWheel::FDMXGDTFWheel(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType)
		: OuterFixtureType(InFixtureType)
	{}

	void FDMXGDTFWheel::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.CreateChildren(TEXT("Slot"), WheelSlotArray);
	}

	FXmlNode* FDMXGDTFWheel::CreateXmlNode(FXmlNode& Parent)
	{
		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.AppendChildren(TEXT("Slot"), WheelSlotArray);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
