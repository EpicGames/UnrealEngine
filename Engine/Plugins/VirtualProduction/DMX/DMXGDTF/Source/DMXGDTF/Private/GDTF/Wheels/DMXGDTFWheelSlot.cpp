// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDTF/Wheels/DMXGDTFWheelSlot.h"

#include "GDTF/Wheels/DMXGDTFAnimationSystem.h"
#include "GDTF/Wheels/DMXGDTFPrismFacet.h"
#include "Serialization/DMXGDTFNodeInitializer.h"
#include "Serialization/DMXGDTFXmlNodeBuilder.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFWheelSlot::FDMXGDTFWheelSlot(const TSharedRef<FDMXGDTFWheel>& InWheel)
		: OuterWheel(InWheel)
	{}

	void FDMXGDTFWheelSlot::Initialize(const FXmlNode& XmlNode)
	{
		FDMXGDTFNodeInitializer(SharedThis(this), XmlNode)
			.GetAttribute(TEXT("Name"), Name)
			.GetAttribute(TEXT("Color"), Color)
			.GetAttribute(TEXT("MediaFileName"), MediaFileName)
			.CreateChildren(TEXT("Facet"), PrismFacetArray)
			.CreateOptionalChild(TEXT("AnimationSystem"), AnimationWheel);
	}

	FXmlNode* FDMXGDTFWheelSlot::CreateXmlNode(FXmlNode& Parent)
	{
		const FString DefaultMediaFileName = TEXT("");

		const FDMXGDTFXmlNodeBuilder ChildBuilder = FDMXGDTFXmlNodeBuilder(Parent, *this)
			.SetAttribute(TEXT("Name"), Name)
			.SetAttribute(TEXT("Color"), Color)
			.SetAttribute(TEXT("MediaFileName"), MediaFileName, DefaultMediaFileName)
			.AppendChildren(TEXT("Facet"), PrismFacetArray)
			.AppendOptionalChild(TEXT("AnimationSystem"), AnimationWheel);

		return ChildBuilder.GetIntermediateXmlNode();
	}
}
