// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "DMXGDTFColorCIE1931xyY.h"
#include "DMXGDTFLog.h"
#include "GDTF/DMXGDTFFixtureType.h"
#include "GDTF/DMXModes/DMXGDTFDMXValue.h"
#include "GDTF/Geometries/DMXGDTFDMXAddress.h"
#include "Internationalization/Regex.h"
#include "Math/Vector2D.h"
#include "Misc/Char.h"
#include "UObject/Class.h"
#include "XmlNode.h"
#include <type_traits>

namespace UE::DMX::GDTF
{
	/**
	 * Initializer for FDMXGDTF nodes.
	 *
	 * To parse non-trivial types GetAttribute accepts transforms and predicates.
	 */
	template<typename NodeType>
	class FDMXGDTFNodeInitializer
		: public FNoncopyable
	{
	public:
		FDMXGDTFNodeInitializer() = delete;
		FDMXGDTFNodeInitializer(TSharedRef<NodeType> InUserObject, const FXmlNode& InXmlNode)
			: UserObject(InUserObject)
			, XmlNode(InXmlNode)
		{
			ensureMsgf(XmlNode.GetTag() == InUserObject->GetXmlTag(), TEXT("Tag mismatch when trying to initialize GDTF node initializer. Expected tag '%s' but got tag '%s'."), InUserObject->GetXmlTag(), *XmlNode.GetTag());
		}
	
		/** Initializes an attribute */
		template <typename AttributeType>
		const FDMXGDTFNodeInitializer& GetAttribute(const TCHAR* InAttribute, AttributeType& OutValue) const
		{
			const FXmlAttribute* AttributePtr = Algo::FindBy(XmlNode.GetAttributes(), InAttribute, &FXmlAttribute::GetTag);
			if (!AttributePtr)
			{
				return *this;
			}
			const FString StringValue = AttributePtr->GetValue();

			if constexpr (std::is_same_v<AttributeType, FString> || std::is_same_v<AttributeType, FName>)
			{
				OutValue = *StringValue;
			}
			else if constexpr (std::is_arithmetic_v<AttributeType>)
			{
				AttributeType ArithmeticValue;
				if (LexTryParseString(ArithmeticValue, *StringValue))
				{
					OutValue = ArithmeticValue;
				}
				else if(!StringValue.IsEmpty())
				{
					UE_LOGF(LogDMXGDTF, Warning, "Failed to parse numerical value from XML attribute '%ls' in node '%ls'. String was '%ls'.", InAttribute, UserObject->GetXmlTag(), *StringValue);
				}
			}
			else if constexpr (std::is_enum_v<AttributeType> && std::is_invocable_v<decltype(&FDMXGDTFNodeInitializer::ParseUEnum<AttributeType>), FDMXGDTFNodeInitializer, const FString&, AttributeType&>)
			{
				AttributeType EnumValue;
				if (ParseUEnum<AttributeType>(StringValue, EnumValue))
				{
					OutValue = EnumValue;
				}
				else
				{
					UE_LOGF(LogDMXGDTF, Warning, "Failed to parse enum value from XML attribute '%ls' in node '%ls'. String was '%ls'.", InAttribute, UserObject->GetXmlTag(), *StringValue);
				}
			}
			else if constexpr (std::is_same_v<AttributeType, FVector2D>)
			{
				OutValue = ParseVector2D(StringValue);
			}
			else if constexpr (std::is_same_v<AttributeType, FVector>)
			{
				OutValue = ParseVector(StringValue);
			}
			else if constexpr (std::is_same_v<AttributeType, FDMXGDTFColorCIE1931xyY>)
			{
				OutValue = ParseColorCIE(StringValue);
			}
			else if constexpr (std::is_same_v<AttributeType, FTransform>)
			{
				OutValue = ParseTransform(StringValue);
			}
			else if constexpr (std::is_same_v<AttributeType, FGuid>)
			{
				OutValue = FGuid(StringValue);
			}
			else if constexpr (std::is_same_v<AttributeType, FDMXGDTFDMXAddress>)
			{
				OutValue = ParseDMXAddress(StringValue);
			}
			else if constexpr (std::is_same_v<AttributeType, FDMXGDTFDMXValue>)
			{
				OutValue = FDMXGDTFDMXValue(*StringValue);
			}
			else
			{
				[] <bool SupportedType = false>()
				{
					static_assert(SupportedType, "Unsupported attribute type.");
				}();
			}

			return *this;
		}

		/** Initializes an attribute, using a static for transformation. */
		template <typename AttributeType, typename UserObject, typename TransformType, typename... Args>
		const FDMXGDTFNodeInitializer& GetAttribute(const TCHAR* InAttribute, AttributeType& OutValue, TransformType Transform, Args...args) const
		{
			const FString StringValue = XmlNode.GetAttribute(InAttribute);
			if (StringValue.IsEmpty())
			{
				return *this;
			}

			OutValue = (*Transform)(*StringValue, args...);

			return *this;
		}

		/** Initializes an attribute, using a member function for transformation. */
		template <typename AttributeType, typename UserObject, typename TransformType, typename... Args>
		const FDMXGDTFNodeInitializer& GetAttribute(const TCHAR* InAttribute, AttributeType& OutValue, const UserObject* Object, TransformType Transform, Args...args) const
		{
			static_assert(std::is_member_function_pointer_v<TransformType>, "TransformType must be a member function pointer.");
			check(Object);

			const FString StringValue = XmlNode.GetAttribute(InAttribute);
			if (StringValue.IsEmpty())
			{
				return *this;
			}

			OutValue = (Object->*Transform)(*StringValue, args...);

			return *this;
		}

		/** Initializes an attribute, using a predicate for transformation. */
		template <typename AttributeType, typename PredicateType>
		const FDMXGDTFNodeInitializer& GetAttribute(const TCHAR* InAttribute, AttributeType& OutValue, PredicateType Predicate) const
		{
			const FString StringValue = XmlNode.GetAttribute(InAttribute);
			if (!StringValue.IsEmpty())
			{
				OutValue = Predicate(*StringValue);
			}
			return *this;
		}

		/** Creates a collection of children nodes, whereas a collection is an xml node that contains these children. */
		template <typename ChildType, typename = std::enable_if_t<std::is_base_of_v<FDMXGDTFNode, ChildType>>>
		const FDMXGDTFNodeInitializer& CreateChildCollection(const TCHAR* InCollectTag, const TCHAR* InChildTag, TArray<TSharedPtr<ChildType>>& OutChildren) const
		{
			if (const FXmlNode* CollectNode = XmlNode.FindChildNode(InCollectTag))
			{
				Algo::TransformIf(CollectNode->GetChildrenNodes(), OutChildren,
					[InChildTag](const FXmlNode* ChildXmlNode)
					{
						return ChildXmlNode && ChildXmlNode->GetTag() == InChildTag;
					},
					[this](const FXmlNode* ChildXmlNode)
					{
						return ConstructChild<ChildType>(ChildXmlNode);
					});
			}
		
			return *this;
		}

		/** Creates child nodes for matching tag in OutChildren array. */
		template <typename ChildType, typename = std::enable_if_t<std::is_base_of_v<FDMXGDTFNode, ChildType>>>
		const FDMXGDTFNodeInitializer& CreateChildren(const TCHAR* InChildTag, TArray<TSharedPtr<ChildType>>& OutChildren) const
		{
			Algo::TransformIf(XmlNode.GetChildrenNodes(), OutChildren,
				[InChildTag](const FXmlNode* ChildXmlNode)
				{
					return ChildXmlNode && ChildXmlNode->GetTag() == InChildTag;
				},
				[this](const FXmlNode* ChildXmlNode)
				{
					return ConstructChild<ChildType>(ChildXmlNode);
				});

			return *this;
		}
	
		/** 
		 * Creates a single child node for matching tag in OutChild. 
		 * Logs a warning if more than one child with this tag are found.
		 */
		template <typename ChildType, typename = std::enable_if_t<std::is_base_of_v<FDMXGDTFNode, ChildType>>>
		const FDMXGDTFNodeInitializer& CreateOptionalChild(const TCHAR* InChildTag, TSharedPtr<ChildType>& OutChild) const
		{
			OutChild = nullptr;

			const FXmlNode* ChildXmlNode = XmlNode.FindChildNode(InChildTag);
			if (ChildXmlNode)
			{
				OutChild = ConstructChild<ChildType>(ChildXmlNode);
			}

			// Log if not unique
			const int32 NumChildren = Algo::CountIf(XmlNode.GetChildrenNodes(),
				[InChildTag](const FXmlNode* ChildXmlNode)
				{
					return ChildXmlNode->GetTag() == InChildTag;
				});
			if (NumChildren > 1)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Trying to parse unique child node '%ls', but found %i child nodes with this tag '%ls'.", InChildTag, NumChildren, InChildTag);
			}

			return *this;
		}

		/** 
		 * Creates a single child node for matching tag in OutChild. Creates the child even if the tag does not exist.
		 * Logs a warning if none or more than one child with this tag are found.
		 */
		template <typename ChildType, typename = std::enable_if_t<std::is_base_of_v<FDMXGDTFNode, ChildType>>>
		const FDMXGDTFNodeInitializer& CreateRequiredChild(const TCHAR* InChildTag, TSharedPtr<ChildType>& OutChild) const
		{
			// Try to create a child from the tag
			CreateOptionalChild(InChildTag, OutChild);

			// Create and log if not constructed from tag
			if (!OutChild.IsValid())
			{
				OutChild = ConstructChild<ChildType>(nullptr);

				UE_LOGF(LogDMXGDTF, Warning, "Failed to parse non-optional child node '%ls' in '%ls'.", InChildTag, UserObject->GetXmlTag());
			}

			return *this;
		}

	private:
		/** Constructs a node */
		template <typename ChildType>
		TSharedRef<ChildType> ConstructChild(const FXmlNode* InXmlNode) const
		{
			const TSharedRef<ChildType> NewNode = MakeShared<ChildType>(UserObject);
		
			if constexpr (std::is_same_v<NodeType, FDMXGDTFFixtureType>)
			{
				// Refer to self when setting the fixture type of the fixture type
				NewNode->WeakFixtureType = UserObject;
			}
			else
			{
				// Refer to parents fixture type
				NewNode->WeakFixtureType = UserObject->GetFixtureType();
			}
			checkf(NewNode->GetFixtureType().IsValid(), TEXT("Failed to construct node. The fixture type is not valid."));

			if (InXmlNode)
			{
				NewNode->Initialize(*InXmlNode);
			}

			return NewNode;
		}

		/** Parses a GDTF string as Enum */
		template <typename EnumType>
		bool ParseUEnum(const FString& GDTFString, EnumType& OutEnum) const
		{
			const UEnum* UEnumObject = StaticEnum<EnumType>();
			int64 Index = UEnumObject->GetValueByName(*GDTFString);
			if (Index != INDEX_NONE)
			{
				OutEnum = static_cast<EnumType>(Index);
				return true;
			}

			return false;
		}

		/** Parses a GDTF string as vector2 */
		FVector2D ParseVector2D(const FString& GDTFString) const
		{
			if (GDTFString.IsEmpty())
			{
				return FVector2D::ZeroVector;
			}

			TArray<FString> Substrings;
			GDTFString.ParseIntoArray(Substrings, TEXT(","));

			if (Substrings.Num() != 2)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF vector 2D. Expected none or two components, but got %i.", Substrings.Num());
				return FVector2D::ZeroVector;
			}

			FVector2D Result;
			bool bSuccess = LexTryParseString(Result.X, *Substrings[0]);
			bSuccess |= LexTryParseString(Result.Y, *Substrings[1]);
			if (!bSuccess)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF vector 2D. Failed to parse %ls.", *GDTFString);
				return FVector2D::ZeroVector;
			}

			return Result;
		}

		/** Parses a GDTF string as vector3 */
		FVector ParseVector(const FString& GDTFString) const
		{
			if (GDTFString.IsEmpty())
			{
				return FVector::ZeroVector;
			}

			const FString VectorString = GDTFString.Replace(TEXT("{"), TEXT("")).Replace(TEXT("}"), TEXT(""));

			TArray<FString> Substrings;
			VectorString.ParseIntoArray(Substrings, TEXT(","));

			if (Substrings.Num() != 3)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF vector. Expected none or three components, but got %i.", Substrings.Num());
				return FVector::ZeroVector;
			}

			FVector Result;
			bool bSuccess = LexTryParseString(Result.X, *Substrings[0]);
			bSuccess |= LexTryParseString(Result.Y, *Substrings[1]);
			bSuccess |= LexTryParseString(Result.Z, *Substrings[2]);
			if (!bSuccess)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF vector 3D. Failed to parse %ls.", *GDTFString);
				return FVector::ZeroVector;
			}

			return Result;
		}

		/** Parses a GDTF string as color CIE */
		FDMXGDTFColorCIE1931xyY ParseColorCIE(const FString& GDTFString) const
		{
			if (GDTFString.IsEmpty())
			{
				return FDMXGDTFColorCIE1931xyY();
			}
			
			TArray<FString> Substrings;
			GDTFString.ParseIntoArray(Substrings, TEXT(","));

			if (Substrings.Num() != 3)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF color. Expected none or three components, but got %i.", Substrings.Num());
				return FDMXGDTFColorCIE1931xyY();
			}

			FDMXGDTFColorCIE1931xyY Result;
			bool bSuccess = LexTryParseString(Result.X, *Substrings[0]);
			bSuccess |= LexTryParseString(Result.Y, *Substrings[1]);
			bSuccess |= LexTryParseString(Result.YY, *Substrings[2]);
			if (!bSuccess)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse GDTF color. Failed to parse %ls.", *GDTFString);
				return FDMXGDTFColorCIE1931xyY();
			}

			return Result;
		}

		/** Parses a GDTF string as transform */
		FTransform ParseTransform(const FString& GDTFString) const
		{
			FMatrix GDTFMatrix;
			if (!ParseGDTFMatrix(GDTFString, GDTFMatrix))
			{
				return FTransform::Identity;
			}
		
			// To Column major order
			const FMatrix ColumnMajorGDTFMatrix = GDTFMatrix.GetTransposed();

			// To Unreal's coordinate system
			const FMatrix GDTFToUnrealMatrix = FMatrix(
				FPlane(1.0, 0.0, 0.0, 0.0),
				FPlane(0.0, 0.0, 1.0, 0.0),
				FPlane(0.0, 1.0, 0.0, 0.0),
				FPlane(0.0, 0.0, 0.0, 1.0)
			);

			const FMatrix UnrealMatrix = GDTFToUnrealMatrix * ColumnMajorGDTFMatrix * GDTFToUnrealMatrix;
			const FTransform Transform = FTransform(UnrealMatrix);

			return Transform;
		}

		/** Parses a GDTF string as DMX address */
		FDMXGDTFDMXAddress ParseDMXAddress(const FString& GDTFString) const
		{
			if (GDTFString.IsEmpty())
			{
				return FDMXGDTFDMXAddress();
			}

			const FString CleanString = GDTFString.Replace(TEXT("{"), TEXT("")).Replace(TEXT("}"), TEXT(""));

			// Can be an absolute adddress
			uint64 AbsoluteAddress;
			if (LexTryParseString(AbsoluteAddress, *CleanString))
			{
				return FDMXGDTFDMXAddress(AbsoluteAddress);
			}

			// Or in the form of Universe.Channel
			TArray<FString> Substrings;
			CleanString.ParseIntoArray(Substrings, TEXT("."));

			if (Substrings.Num() == 2)
			{
				int32 Universe;
				int32 Channel;
				if (LexTryParseString(Universe, *Substrings[0]) &&
					LexTryParseString(Channel, *Substrings[1]))
				{
					constexpr uint16 UniverseSize = 512;

					FDMXGDTFDMXAddress Result;
					Result.AbsoluteAddress = Universe * UniverseSize + Channel - 1;
					return Result;
				}
			}

			UE_LOGF(LogDMXGDTF, Warning, "Failed to parse DMX Address. '%ls' is not a valid GDTF string", *GDTFString);
			return FDMXGDTFDMXAddress();
		}

		/** Helper to parse 3x3 and 4x4 matrices */
		bool ParseGDTFMatrix(const FString& InGDTFString, FMatrix& OutMatrix) const
		{
			if (InGDTFString.IsEmpty())
			{
				return false;
			}

			// Parse rows
			TArray<FString> RowStrings;

			const FRegexPattern CurlyBracketRegex(TEXT("\\{([^}]+)\\}"), ERegexPatternFlags::CaseInsensitive);
			FRegexMatcher CurlyBracketMatcher(CurlyBracketRegex, InGDTFString);
			while (CurlyBracketMatcher.FindNext())
			{
				RowStrings.Add(CurlyBracketMatcher.GetCaptureGroup(1));
			}

			// Parse each component
			auto GetComponentsFromStringLambda([](const FString& String)
				{
					TArray<FString> Substrings;
					String.ParseIntoArray(Substrings, TEXT(","));

					TArray<float> Result;
					Result.AddZeroed(Substrings.Num());
					for (int32 StringIndex = 0; StringIndex < Substrings.Num(); StringIndex++)
					{
						LexTryParseString(Result[StringIndex], *Substrings[StringIndex]);
					}
					return Result;
				});
			const TArray<float> U = GetComponentsFromStringLambda(RowStrings[0]);
			const TArray<float> V = GetComponentsFromStringLambda(RowStrings[1]);
			const TArray<float> W = GetComponentsFromStringLambda(RowStrings[2]);
			const TArray<float> O = RowStrings.IsValidIndex(3) ? GetComponentsFromStringLambda(RowStrings[3]) : TArray<float>({ 0.f, 0.f, 0.f });

			// Only 3x3 or 4x4 is supported
			const uint32 NumRows = RowStrings.Num();
			const bool bValid = (U.Num() == NumRows && V.Num() == NumRows && W.Num() == NumRows && O.Num() == NumRows);
			if (!bValid)
			{
				UE_LOGF(LogDMXGDTF, Warning, "Cannot parse 3x3 or 4x4 matrix from GDTF string '%ls'.", *InGDTFString);
				return false;
			}

			OutMatrix = FMatrix(
				FPlane(U[0], U[1], U[2], U.IsValidIndex(3) ? U[3] : 0.f),
				FPlane(V[0], V[1], V[2], V.IsValidIndex(3) ? V[3] : 0.f),
				FPlane(W[0], W[1], W[2], W.IsValidIndex(3) ? W[3] : 0.f),
				FPlane(O[0], O[1], O[2], O.IsValidIndex(3) ? O[3] : 1.f)
			);

			return true;
		}

		/** The object that uses this parser */
		TSharedRef<NodeType> UserObject;

		/** The XML node that needs to be parsed */
		const FXmlNode& XmlNode;
	};
}
