// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/DMXGDTFXmlNodeBuilder.h"

#include "Algo/Sort.h"
#include "XmlNode.h"

namespace UE::DMX::GDTF
{
	FDMXGDTFXmlNodeBuilder::FDMXGDTFXmlNodeBuilder(FXmlNode& InOutParent, const FDMXGDTFNode& InGDTFNode, FXmlNode* AppendToNode)
		: Parent(InOutParent)
	{
		if (AppendToNode)
		{
			NewXmlNode = AppendToNode;
			Attributes = AppendToNode->GetAttributes();
		}
		else
		{
			Parent.AppendChildNode(InGDTFNode.GetXmlTag());
			NewXmlNode = Parent.GetChildrenNodes().Last();
		}
	}

	FDMXGDTFXmlNodeBuilder::~FDMXGDTFXmlNodeBuilder()
	{
		check(NewXmlNode);

		Algo::SortBy(Attributes, &FXmlAttribute::GetTag);
		NewXmlNode->SetAttributes(Attributes);
	}

#define SKIP_IF_OPTIONAL_AND_DEFAULT(Value) \
	if (OptionalDefault.IsSet() && OptionalDefault.GetValue() == Value) \
	{ \
		return *this; \
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const TCHAR* CharArray, const TOptional<FString> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(CharArray);

		Attributes.Add(FXmlAttribute(AttributeName, CharArray));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FString& String, const TOptional<FString> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(String);

		Attributes.Add(FXmlAttribute(AttributeName, String));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FName& Name, const TOptional<FName> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(Name);

		const FString NameAsString = Name.IsNone() ? TEXT("") : Name.ToString();

		Attributes.Add(FXmlAttribute(AttributeName, NameAsString));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const int32 Integer, const TOptional<int32> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(Integer);

		Attributes.Add(FXmlAttribute(AttributeName, *FString::Printf(TEXT("%i"), Integer)));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const uint32 UnsignedInteger, const TOptional<uint32> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(UnsignedInteger);

		Attributes.Add(FXmlAttribute(AttributeName, *FString::Printf(TEXT("%u"), UnsignedInteger)));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const float Float, const TOptional<float> OptionalDefault)
	{
		if (OptionalDefault.IsSet() && FMath::IsNearlyEqual(Float, OptionalDefault.GetValue(), FLT_EPSILON))
		{
			return *this;
		}

		constexpr int32 SignificantDigits = 6;
		Attributes.Add(FXmlAttribute(AttributeName, FString::SanitizeFloat(Float, SignificantDigits)));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FGuid& Guid, const TOptional<FGuid> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(Guid);

		if (Guid.IsValid())
		{
			Attributes.Add(FXmlAttribute(AttributeName, Guid.ToString(EGuidFormats::DigitsWithHyphens)));
		}
		else
		{
			Attributes.Add(FXmlAttribute(AttributeName, TEXT("")));
		}

		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FVector2D& Vector2, const TOptional<FVector2D> OptionalDefault)
	{
		if (OptionalDefault.IsSet() && Vector2.Equals(OptionalDefault.GetValue(), FLT_EPSILON))
		{
			return *this;
		}

		constexpr int32 SignificantDigits = 6;
		const FString X = FString::SanitizeFloat(Vector2.X, SignificantDigits);
		const FString Y = FString::SanitizeFloat(Vector2.Y, SignificantDigits);

		Attributes.Add(FXmlAttribute(AttributeName, FString::Printf(TEXT("{%s,%s}"), *X, *Y)));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FVector& Vector3, const TOptional<FVector> OptionalDefault)
	{
		if (OptionalDefault.IsSet() && Vector3.Equals(OptionalDefault.GetValue(), FLT_EPSILON))
		{
			return *this;
		}

		constexpr int32 SignificantDigits = 6;
		const FString X = FString::SanitizeFloat(Vector3.X, SignificantDigits);
		const FString Y = FString::SanitizeFloat(Vector3.Y, SignificantDigits);
		const FString Z = FString::SanitizeFloat(Vector3.Z, SignificantDigits);

		Attributes.Add(FXmlAttribute(AttributeName, FString::Printf(TEXT("{%s,%s,%s}"), *X, *Y, *Z)));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FDMXGDTFDMXValue& DMXValue, const TOptional<FDMXGDTFDMXValue> OptionalDefault)
	{
		if (OptionalDefault.IsSet() && DMXValue == OptionalDefault.GetValue())
		{
			return *this;
		}

		Attributes.Add(FXmlAttribute(AttributeName, DMXValue.AsString()));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FDMXGDTFDMXAddress& GDTFAddress, const TOptional<FDMXGDTFDMXAddress> OptionalDefault)
	{
		if (OptionalDefault.IsSet() && GDTFAddress.AbsoluteAddress == OptionalDefault.GetValue().AbsoluteAddress)
		{
			return *this;
		}

		// Absolute if possible
		constexpr uint16 UniverseSize = 512;
		const uint64 AbsoluteAddress = GDTFAddress.GetUniverse() * UniverseSize + GDTFAddress.GetChannel();
		if (AbsoluteAddress <= std::numeric_limits<uint32>::max())
		{
			Attributes.Add(FXmlAttribute(AttributeName, FString::Printf(TEXT("%lli"), AbsoluteAddress)));
		}
		else
		{
			const FString Universe = FString::FromInt(GDTFAddress.GetUniverse());
			const FString Channel = FString::FromInt(GDTFAddress.GetChannel());

			Attributes.Add(FXmlAttribute(AttributeName, FString::Printf(TEXT("{%s.%s}"), *Universe, *Channel)));
		}

		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FDateTime& DateTime, const TOptional<FDateTime> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(DateTime);

		if (FDateTime::Validate(DateTime.GetYear(), DateTime.GetMonth(), DateTime.GetDay(), DateTime.GetHour(), DateTime.GetMinute(), DateTime.GetSecond(), DateTime.GetMillisecond()))
		{
			Attributes.Add(FXmlAttribute(AttributeName, DateTime.ToIso8601()));
		}
		else
		{
			UE_LOGF(LogDMXGDTF, Warning, "Failed to write DateTime to GDTF description. Current values do not form a valid DateTime");
		}

		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FTransform& Transform, EDMXGDTFMatrixType MatrixType, const TOptional<FTransform> OptionalDefault)
	{
		if (OptionalDefault.IsSet() && OptionalDefault.GetValue().Equals(Transform, FLT_EPSILON))
		{
			return *this;
		}

		FMatrix UnrealMatrix = Transform.ToMatrixWithScale();
		const FMatrix TransposedUnrealMatrix = UnrealMatrix.GetTransposed();

		// From Unreal's coordinate system to GDTF's coordinate system
		const FMatrix GDTFToUnrealMatrix = FMatrix(
			FPlane(1.0, 0.0, 0.0, 0.0),
			FPlane(0.0, 0.0, 1.0, 0.0),
			FPlane(0.0, 1.0, 0.0, 0.0),
			FPlane(0.0, 0.0, 0.0, 1.0)
		);

		FMatrix GDTFMatrix = GDTFToUnrealMatrix * TransposedUnrealMatrix * GDTFToUnrealMatrix;

		auto Sanetize = [GDTFMatrix](uint8 x, uint8 y)
			{		
				constexpr int32 SignificantDigits = 6;
				return FString::SanitizeFloat(GDTFMatrix.M[x][y], SignificantDigits);
			};

		FString Attribute;
		if (MatrixType == EDMXGDTFMatrixType::Matrix3x3)
		{

			Attribute += FString::Printf(TEXT("{%s,%s,%s}"), *Sanetize(0, 0), *Sanetize(0, 1), *Sanetize(0, 2));
			Attribute += FString::Printf(TEXT("{%s,%s,%s}"), *Sanetize(1, 0), *Sanetize(1, 1), *Sanetize(1, 2));
			Attribute += FString::Printf(TEXT("{%s,%s,%s}"), *Sanetize(2, 0), *Sanetize(2, 1), *Sanetize(2, 2));
		}
		else if (MatrixType == EDMXGDTFMatrixType::Matrix4x4)
		{
			Attribute += FString::Printf(TEXT("{%s,%s,%s,%s}"), *Sanetize(0, 0), *Sanetize(0, 1), *Sanetize(0, 2), *Sanetize(0, 3));
			Attribute += FString::Printf(TEXT("{%s,%s,%s,%s}"), *Sanetize(1, 0), *Sanetize(1, 1), *Sanetize(1, 2), *Sanetize(1, 3));
			Attribute += FString::Printf(TEXT("{%s,%s,%s,%s}"), *Sanetize(2, 0), *Sanetize(2, 1), *Sanetize(2, 2), *Sanetize(2, 3));
			Attribute += FString::Printf(TEXT("{%s,%s,%s,%s}"), *Sanetize(3, 0), *Sanetize(3, 1), *Sanetize(3, 2), *Sanetize(3, 3));
		}
		else
		{
			checkf(0, TEXT("Unhandled enum value"));
		}

		Attributes.Add(FXmlAttribute(AttributeName, Attribute));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const FDMXGDTFColorCIE1931xyY& ColorCIE, const TOptional<FDMXGDTFColorCIE1931xyY> OptionalDefault)
	{
		SKIP_IF_OPTIONAL_AND_DEFAULT(ColorCIE);

		Attributes.Add(FXmlAttribute(AttributeName, ColorCIE.ToString()));
		return *this;
	}

	FDMXGDTFXmlNodeBuilder& FDMXGDTFXmlNodeBuilder::SetAttribute(const FString& AttributeName, const TArray<FDMXGDTFColorCIE1931xyY>& ColorArray)
	{
		if (ColorArray.IsEmpty())
		{
			return *this;
		}

		FString ColorString;
		for (const FDMXGDTFColorCIE1931xyY& Color : ColorArray)
		{
			constexpr int32 SignificantDigits = 6;
			const FString XString = FString::SanitizeFloat(Color.X, SignificantDigits);
			const FString YString = FString::SanitizeFloat(Color.Y, SignificantDigits);
			const FString LuminanceString = FString::SanitizeFloat(Color.YY, SignificantDigits);

			ColorString += FString::Printf(TEXT("{%s,%s,%s}"), *XString, *YString, *LuminanceString);
		}
		Attributes.Add(FXmlAttribute(AttributeName, ColorString));

		return *this;
	}

	FXmlNode* FDMXGDTFXmlNodeBuilder::GetIntermediateXmlNode() const
	{
		check(NewXmlNode);

		return NewXmlNode;
	}

#undef SKIP_IF_OPTIONAL_AND_DEFAULT
}
