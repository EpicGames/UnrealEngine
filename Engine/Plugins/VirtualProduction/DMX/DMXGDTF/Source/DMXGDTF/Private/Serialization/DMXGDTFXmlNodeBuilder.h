// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DMXGDTFColorCIE1931xyY.h"
#include "DMXGDTFLog.h"
#include "GDTF/DMXGDTFNode.h"
#include "GDTF/DMXModes/DMXGDTFDMXValue.h"
#include "GDTF/Geometries/DMXGDTFDMXAddress.h"
#include "Misc/Optional.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "XmlNode.h"
#include <type_traits>

namespace UE::DMX::GDTF
{
	struct FDMXGDTFDMXAddress;
	struct FDMXGDTFDMXValue;

	enum class EDMXGDTFMatrixType
	{
		Matrix3x3,
		Matrix4x4
	};

	class FDMXGDTFXmlNodeBuilder
	{
	public:
		/** 
		 * Builds a GDTF XML node from an Unreal DMXGDTF node. 
		 * 
		 * @param InOutParent		The parent XML node the new node is built in
		 * @param InGDTFNode		The DMX GDTF node that from which the XML node is created
		 * @param AppendToNode		(optional) If not nullptr, appends to an existing XML node
		 */
		FDMXGDTFXmlNodeBuilder(FXmlNode& InOutParent, const FDMXGDTFNode& InGDTFNode, FXmlNode* AppendToNode = nullptr);
		~FDMXGDTFXmlNodeBuilder();

		/** Converts a char array to an attribute. If a default value is provided, only sets the attribute if the char array differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const TCHAR* CharArray, const TOptional<FString> OptionalDefault = TOptional<FString>());

		/** Converts a string to an attribute. If a default value is provided, only sets the attribute if the string differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FString& String, const TOptional<FString> OptionalDefault = TOptional<FString>());

		/** Converts a name to an attribute. If a default value is provided, only sets the attribute if the name differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FName& Name, const TOptional<FName> OptionalDefault = TOptional<FName>());
	
		/** Converts an integer to an attribute. If a default value is provided, only sets the attribute if the integer differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const int32 Integer, const TOptional<int32> OptionalDefault = TOptional<int32>());
		
		/** Converts an unsigned integer to an attribute. If a default value is provided, only sets the attribute if the unsigned integer differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const uint32 UnsignedInteger, const TOptional<uint32> OptionalDefault = TOptional<uint32>());

		/** Converts a float to an attribute. If a default value is provided, only sets the attribute if the float differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const float Float, const TOptional<float> OptionalDefault = TOptional<float>());

		/** Converts a guid to an attribute. If a default value is provided, only sets the attribute if the guid differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FGuid& Guid, const TOptional<FGuid> OptionalDefault = TOptional<FGuid>());
	
		/** Converts a vector2 to an attribute. If a default value is provided, only sets the attribute if the vector2 differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FVector2D& Vector2, const TOptional<FVector2D> OptionalDefault = TOptional<FVector2D>());

		/** Converts a vector3 to an attribute. If a default value is provided, only sets the attribute if the vector3 differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FVector& Vector3, const TOptional<FVector> OptionalDefault = TOptional<FVector>());
	
		/** 
		 * Converts a transform to an attribute. If a default value is provided, only sets the attribute if the transform differs from the default value.
		 * 
		 * This setter requires to set if a 3x3 or 4x4 matrix should be written.
		 */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FTransform& Transform, EDMXGDTFMatrixType MatrixType, const TOptional<FTransform> OptionalDefault = TOptional<FTransform>());

		/** Converts a color to an attribute. If a default value is provided, only sets the attribute if the color differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FDMXGDTFColorCIE1931xyY& ColorCIE, const TOptional<FDMXGDTFColorCIE1931xyY> OptionalDefault = TOptional<FDMXGDTFColorCIE1931xyY>());

		/** Converts an array of colors to an attribute. If a default value is provided, only sets the attribute if the array of colors differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const TArray<FDMXGDTFColorCIE1931xyY>& ColorArray);

		/** Converts a GDTF value to an attribute. If a default value is provided, only sets the attribute if the GDTF value differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FDMXGDTFDMXValue& GDTFValue, const TOptional<FDMXGDTFDMXValue> OptionalDefault = TOptional<FDMXGDTFDMXValue>());

		/** Converts a GDTF address to an attribute. If a default value is provided, only sets the attribute if the GDTF address differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FDMXGDTFDMXAddress& GDTFAddress, const TOptional<FDMXGDTFDMXAddress> OptionalDefault = TOptional<FDMXGDTFDMXAddress>());

		/** Converts date time to an attribute. If a default value is provided, only sets the attribute if the date time differs from the default value */
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const FDateTime& DateTime, const TOptional<FDateTime> OptionalDefault = TOptional<FDateTime>());

		/** Converts an array of integer to an attribute. If a default value is provided, only sets the attribute if the array of integer differs from the default value */
		template <typename IntegralType> requires std::is_integral_v<IntegralType> && std::negation_v<std::is_enum<IntegralType>>
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const TArray<IntegralType> ValueArray, const TOptional<FString> OptionalDefault = TOptional<FString>())
		{
			if (ValueArray.IsEmpty() && OptionalDefault.IsSet())
			{
				Attributes.Add(FXmlAttribute(AttributeName, OptionalDefault.GetValue()));
				return *this;
			}

			FString String;
			for (const IntegralType& Value : ValueArray)
			{
				String += FString::FromInt(Value);

				if (Value != ValueArray.Last())
				{
					String += TEXT(",");
				}
			}

			Attributes.Add(FXmlAttribute(AttributeName, String));
			return *this;
		}

		/** 
		 * Converts an enum to an attribute. Note the enum has to be a UEnum value
		 * If a default value is provided, only sets the enum if the vector differs from the default value 
		 */
		template <typename EnumType> requires std::is_enum_v<EnumType> && std::is_invocable_v<decltype(&StaticEnum<EnumType>)> && std::negation_v<std::is_integral<EnumType>>
		FDMXGDTFXmlNodeBuilder& SetAttribute(const FString& AttributeName, const EnumType& EnumValue, const TOptional<EnumType> OptionalDefault = TOptional<EnumType>())
		{
			if (OptionalDefault.IsSet() && OptionalDefault.GetValue() == EnumValue)
			{
				return *this;
			}

			const UEnum* EnumClass = StaticEnum<EnumType>();
			check(EnumClass);
			const FString StringValue = EnumClass->GetNameStringByValue(static_cast<int64>(EnumValue));
			if (!StringValue.IsEmpty())
			{
				Attributes.Add(FXmlAttribute(AttributeName, StringValue));
			}
			else
			{
				UE_LOGF(LogDMXGDTF, Warning, "Failed to convert UEnum value to string with UEnum '%ls'", *EnumClass->GetName());
			}

			return *this;
		}

		template <typename NodeType> requires std::is_base_of_v<FDMXGDTFNode, NodeType>
		FDMXGDTFXmlNodeBuilder& AppendRequiredChild(const TCHAR* ChildTag, const TSharedPtr<NodeType>& ChildNode)
		{
			check(NewXmlNode);

			if (ChildNode.IsValid() && 
				ensureMsgf(ChildNode->GetXmlTag() == ChildTag, TEXT("Tag mismatch when trying to build GDTF XML node. Expected tag '%s' but got tag '%s'."), *Parent.GetTag(), ChildTag))
			{
				ChildNode->CreateXmlNode(*NewXmlNode);
			}
			else
			{
				UE_LOGF(LogDMXGDTF, Warning, "Invalid non-optional child node '%ls' in '%ls'. Generating a default node instead.", ChildTag, *Parent.GetTag());
				Parent.AppendChildNode(ChildTag);
			}

			return *this;
		}

		template <typename NodeType> requires std::is_base_of_v<FDMXGDTFNode, NodeType>
		FDMXGDTFXmlNodeBuilder& AppendOptionalChild(const TCHAR* ChildTag, const TSharedPtr<NodeType>& ChildNode)
		{
			check(NewXmlNode);

			if (ChildNode.IsValid())
			{
				ChildNode->CreateXmlNode(*NewXmlNode);
			}

			return *this;
		}

		template <typename NodeType> requires std::is_base_of_v<FDMXGDTFNode, NodeType>
		FDMXGDTFXmlNodeBuilder& AppendChildren(const TCHAR* ChildTag, const TArray<TSharedPtr<NodeType>>& ChildNodes)
		{
			check(NewXmlNode);

			for (const TSharedPtr<NodeType>& ChildNode : ChildNodes)
			{
				if (ChildNode.IsValid())
				{
					ChildNode->CreateXmlNode(*NewXmlNode);
				}
			}

			return *this;
		}

		template <typename NodeType> requires std::is_base_of_v<FDMXGDTFNode, NodeType>
		FDMXGDTFXmlNodeBuilder& AppendChildCollection(const TCHAR* CollectName, const TCHAR* ChildTag, const TArray<TSharedPtr<NodeType>>& ChildNodes)
		{
			check(NewXmlNode);

			NewXmlNode->AppendChildNode(CollectName);
			FXmlNode* CollectNode = NewXmlNode->GetChildrenNodes().Last();
			checkf(CollectNode, TEXT("Failed to create child collect in %s"), *Parent.GetTag());

			for (const TSharedPtr<NodeType>& ChildNode : ChildNodes)
			{
				ChildNode->CreateXmlNode(*CollectNode);
			}
			return *this;
		}

		/** Returns the intermediate XML node. Note the node will exist, but will not yet contain the XML data */
		FXmlNode* GetIntermediateXmlNode() const;

	private:
		/** Array of attributes that are pending to be written */
		TArray<FXmlAttribute> Attributes;

		/** The parent node in which this node is built */
		FXmlNode& Parent;

		/** The XML node that is being built */
		FXmlNode* NewXmlNode = nullptr;
	};
}
