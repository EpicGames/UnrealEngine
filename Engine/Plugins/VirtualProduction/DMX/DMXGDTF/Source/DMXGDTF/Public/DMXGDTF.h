// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"

#include "DMXGDTF.generated.h"

class FXmlFile;
namespace UE::DMX::GDTF 
{ 
	class FDMXGDTFDescription; 
	class FDMXGDTFFixtureType;
}

/** 
 * The implementation of the GDTF standard in Unreal Engine. 
 * 
 * The implementation uses unreal engine types, left-handed z-up coordinate system, centimeters, column-major order matrices. .
 */
UCLASS()
class DMXGDTF_API UDMXGDTF
	: public UObject
{
	GENERATED_BODY()

	using FDMXGDTFDescription = UE::DMX::GDTF::FDMXGDTFDescription;
	using FDMXGDTFFixtureType = UE::DMX::GDTF::FDMXGDTFFixtureType;

public:
	/** Initializes this object from .gdtf file data. Returns true on success. */
	void InitializeFromData(const TArray64<uint8>& Data);

	/** Initializes this object from a GDTF fixture type. */
	void InitializeFromFixtureType(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

	/** Exports an XML file. Returns the XML file or nullptr if no XML file could be generated */
	TSharedPtr<FXmlFile> ExportAsXml() const;

	/** Returns the GDTF Description or nullptr if no valid description exists. */
	TSharedPtr<FDMXGDTFDescription> GetDescription() const { return Description; }

private:
	/** The GDTF description */
	TSharedPtr<FDMXGDTFDescription> Description;
};
