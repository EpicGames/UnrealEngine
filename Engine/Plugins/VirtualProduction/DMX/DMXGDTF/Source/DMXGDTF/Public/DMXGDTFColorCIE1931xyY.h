// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.generated.h"

/** xyY color representation in the CIE 1931 color space, as typically used in GDTF and MVR */
USTRUCT(BlueprintType, Category = "DMX")
struct DMXGDTF_API FDMXGDTFColorCIE1931xyY
{
	GENERATED_BODY()

	FString ToString() const;

	/** Chromaticity coordinate x */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	float X = 0.f;

	/** Chromaticity coordinate y */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	float Y = 0.f;

	/** Luminance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX", Meta = (DisplayName = "Luminance"))
	float YY = 0.f;

	friend bool operator==(const FDMXGDTFColorCIE1931xyY& A, const FDMXGDTFColorCIE1931xyY& B)
	{
		return A.X == B.X && A.Y == B.Y && A.YY == B.YY;
	}

	friend bool operator!=(const FDMXGDTFColorCIE1931xyY& A, const FDMXGDTFColorCIE1931xyY& B)
	{
		return !(A == B);
	}

	friend FArchive& operator<<(FArchive& Ar, FDMXGDTFColorCIE1931xyY& ColorCIE1931)
	{
		Ar << ColorCIE1931.X;
		Ar << ColorCIE1931.Y;
		Ar << ColorCIE1931.YY;
		return Ar;
	}
};
