// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFAttributeDefinitions;

	/** This section defines the activation group Attributes (XML node <ActivationGroup>). */
	class DMXGDTF_API FDMXGDTFActivationGroup
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFActivationGroup(const TSharedRef<FDMXGDTFAttributeDefinitions>& InAttributeDefinitions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("ActivationGroup"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the activation group. */
		FName Name;

		/** The outer attribute definitions */
		const TWeakPtr<FDMXGDTFAttributeDefinitions> OuterAttributeDefinitions;
	};
}
