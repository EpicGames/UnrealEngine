// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"

enum class EDMXGDTFPhysicalUnit : uint8;

/** This section defines the Fixture Type Attributes (XML node <Attributes>). As children, the attributes node has a list of attributes. */
namespace UE::DMX::GDTF
{
	class FDMXGDTFActivationGroup;
	class FDMXGDTFAttributeDefinitions;
	class FDMXGDTFFeature;
	class FDMXGDTFSubphysicalUnit;

	class DMXGDTF_API FDMXGDTFAttribute
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFAttribute(const TSharedRef<FDMXGDTFAttributeDefinitions>& InAttributeDefinitions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Attribute"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the attribute. */
		FName Name;

		/** The pretty name of the attribute. */
		FString Pretty;

		/** (Optional) Link to the activation group. */
		FString ActivationGroup;

		/** (Optional) Link to the corresponding feature. */
		FString Feature;

		/** (Optional) Link to the main attribute. */
		FString MainAttribute;

		/** Physical Unit */
		EDMXGDTFPhysicalUnit PhysicalUnit;

		/** (Optional) Defines the color for the attribute. */
		FDMXGDTFColorCIE1931xyY Color;

		/** As children the attribute node has a list of a subphysical units. */
		TArray<TSharedPtr<FDMXGDTFSubphysicalUnit>> SubpyhsicalUnitArray;

		/** The outer attribute definitions */
		const TWeakPtr<FDMXGDTFAttributeDefinitions> OuterAttributeDefinitions;

		/** Resolves the linked activation group. Returns the activation group or nullptr if no activation group is linked */
		TSharedPtr<FDMXGDTFActivationGroup> ResolveActivationGroup() const;

		/** Resolves the linked feature. Returns the feature or nullptr if no feature is linked */
		TSharedPtr<FDMXGDTFFeature> ResolveFeature() const;

		/** Resolves the linked main attribute. Returns the main attribute or nullptr if no main attribute is linked */
		TSharedPtr<FDMXGDTFAttribute> ResolveMainAttribute() const;
	};
}
