// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFActivationGroup;
	class FDMXGDTFAttribute;
	class FDMXGDTFFeature;
	class FDMXGDTFFeatureGroup;
	class FDMXGDTFFixtureType;

	/** This section defines the attribute definition collect for the Fixture Type Attributes. */
	class DMXGDTF_API FDMXGDTFAttributeDefinitions
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFAttributeDefinitions(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("AttributeDefinitions"); }
		virtual void Initialize(const FXmlNode& InXmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** 
		 * (Optional) This section defines groups of Fixture Type Attributes that are intended to be used together.
		 * Example: Usually Pan and Tilt are Fixture Type Attributes that shall be activated together to be able to store and recreate any position.
		 * The current activation groups node does not have any XML attributes (XML node <ActivationGroups>). As children it can have a list of a activation group.
		 */
		TArray<TSharedPtr<FDMXGDTFActivationGroup>> ActivationGroups;

		/** 
		 * (Optional) This section defines the logical grouping of Fixture Type Attributes (XML node <FeatureGroups>).
		 * [For example, Gobo 1 and Gobo 2 are grouped in the feature Gobo of the feature group Gobo.]
		 *
		 * NOTE 1: A feature group can contain more than one logical control unit.
		 * A feature group Position shall contain PanTilt and XYZ as separate Feature.
		 *
		 * NOTE 2: Usually Pan and Tilt create a logical unit to enable position control, so they must be grouped in a Feature PanTilt.
		 */
		TArray<TSharedPtr<FDMXGDTFFeatureGroup>> FeatureGroups;

		/** This section defines the Fixture Type Attributes (XML node <Attributes>). As children the attributes node has a list of a attributes. */
		TArray<TSharedPtr<FDMXGDTFAttribute>> Attributes;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;

		/** Finds an activation group by name. Returns the activation group or nullptr if the activation group cannot be found */
		TSharedPtr<FDMXGDTFActivationGroup> FindActivationGroup(const FString& Name) const;

		/** Finds an attribute by name. Returns the attribute or nullptr if the attribute cannot be found */
		TSharedPtr<FDMXGDTFAttribute> FindAttribute(const FString& Name) const;

		/** Finds a feature by name. Returns the feature or nullptr if the feature cannot be found */
		TSharedPtr<FDMXGDTFFeature> FindFeature(const FString& FeatureGroupName, const FString& FeatureName) const;
	};
}
