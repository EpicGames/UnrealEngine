// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFeatureGroup;

	/** This section defines the feature. */
	class DMXGDTF_API FDMXGDTFFeature
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFFeature(const TSharedRef<FDMXGDTFFeatureGroup>& InFeatureGroup);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Feature"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the feature. */
		FName Name;

		/** The outer feature group */
		const TWeakPtr<FDMXGDTFFeatureGroup> OuterFeatureGroup;
	};
}
