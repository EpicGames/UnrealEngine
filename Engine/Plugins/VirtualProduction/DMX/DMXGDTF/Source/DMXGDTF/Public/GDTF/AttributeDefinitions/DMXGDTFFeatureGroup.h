// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFAttributeDefinitions;
	class FDMXGDTFFeature;
	
	/** This section defines the feature group (XML node <FeatureGroup>). */
	class DMXGDTF_API FDMXGDTFFeatureGroup
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFFeatureGroup(const TSharedRef<FDMXGDTFAttributeDefinitions>& InAttributeDefinitions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FeatureGroup"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the feature group. */
		FName Name;

		/** The pretty name of the feature group. */
		FString Pretty;

		/** As children the feature group has a list of a feature. */
		TArray<TSharedPtr<FDMXGDTFFeature>> FeatureArray;

		/** The outer attribute definitions */
		const TWeakPtr<FDMXGDTFAttributeDefinitions> OuterAttributeDefinitions;
	};
}
