// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumRange.h"

#include "DMXGDTFPhysicalUnit.generated.h"

/**
 * Attribute Pyhsical Unit.
 *
 * The currently defined unit values are: “None”, “Percent”, “Length” (m), “Mass” (kg), “Time” (s), “Temperature” (K), “LuminousIntensity”(cd),
 * “Angle” (degree), “Force” (N), “Frequency” (Hz), “Current” (A), “Voltage” (V), “Power” (W), “Energy” (J), “Area” (m2), “Volume” (m3), “Speed” (m/s),
 * “Acceleration” (m/s2), “AngularSpeed” (degree/s), “AngularAccc” (degree/s2), “WaveLength” (nm), “ColorComponent”. Default: “None”.
 */
UENUM(BlueprintType)
enum class EDMXGDTFPhysicalUnit : uint8
{
	None UMETA(DisplayName = "None"),
	Percent UMETA(DisplayName = "Percent"),
	Length UMETA(DisplayName = "Length"),
	Mass UMETA(DisplayName = "Mass (kg)"),
	Time UMETA(DisplayName = "Time (s)"),
	Temperature UMETA(DisplayName = "Temperature (K)"),
	LuminousIntensity UMETA(DisplayName = "LuminousIntensity (cd)"),
	Angle UMETA(DisplayName = "Angle (degree)"),
	Force UMETA(DisplayName = "Force (N)"),
	Frequency UMETA(DisplayName = "Frequency (Hz)"),
	Current UMETA(DisplayName = "Current (A)"),
	Voltage UMETA(DisplayName = "Voltage (V)"),
	Power UMETA(DisplayName = "Power (W)"),
	Energy UMETA(DisplayName = "Energy (J)"),
	Area UMETA(DisplayName = "Area (m2)"),
	Volume UMETA(DisplayName = "Volume (m3)"),
	Speed UMETA(DisplayName = "Speed (m/s)"),
	Acceleration UMETA(DisplayName = "Acceleration (m/s2)"),
	AngularSpeed UMETA(DisplayName = "AngularSpeed (degree/s)"),
	AngularAccc UMETA(DisplayName = "AngularAccc (degree/s2"),
	WaveLength UMETA(DisplayName = "WaveLength (nm)"),
	ColorComponent UMETA(DisplayName = "ColorComponent"),

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFPhysicalUnit, EDMXGDTFPhysicalUnit::MaxEnumValue);
