// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFSubphysicalUnit.generated.h"

enum class EDMXGDTFPhysicalUnit : uint8;

/** 
 * Subphysical Unit Type.
 * The currently defined values are : “PlacementOffset”, “Amplitude”,
 * “AmplitudeMin”, “AmplitudeMax”, “Duration”, “DutyCycle”, “TimeOffset”,
 * “MinimumOpening”, “Value”, “RatioHorizontal”, “RatioVertical”.
 */
UENUM(BlueprintType)
enum class EDMXGDTFSubphysicalUnit : uint8
{
	PlacementOffset,
	Amplitude,
	AmplitudeMin,
	AmplitudeMax,
	Duration,
	DutyCycle,
	TimeOffset,
	MinimumOpening,
	Value,
	RatioHorizontal,
	RatioVertical,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFSubphysicalUnit, EDMXGDTFSubphysicalUnit::MaxEnumValue);

/** This section defines the Attribute Subphysical Unit(XML node <SubPhysicalUnit>). */
namespace UE::DMX::GDTF
{
	class FDMXGDTFAttribute;

	class DMXGDTF_API FDMXGDTFSubphysicalUnit
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFSubphysicalUnit(const TSharedRef<FDMXGDTFAttribute>& InAttribute);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("SubphysicalUnit"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Type of subpyhsical unit */
		EDMXGDTFSubphysicalUnit Type = EDMXGDTFSubphysicalUnit::Value;

		/** Phyiscal Unit */
		EDMXGDTFPhysicalUnit PhysicalUnit;

		/** The default physical from of the subphysical unit; Unit: as defined in PhysicalUnit; Default value: 0 */
		float PhysicalFrom = 0.0;

		/** The default physical to of the subphysical unit; Unit: as defined in PhysicalUnit; Default value: 1 */
		float PhysicalTo = 1.0;

		/** The outer attribute */
		const TWeakPtr<FDMXGDTFAttribute> OuterAttribute;
	};
}
