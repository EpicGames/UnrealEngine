// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFVersion.h"
#include "Templates/SharedPointer.h"

class FXmlFile;

namespace UE::DMX::GDTF
{
	class FDMXGDTFNode;
	class FDMXGDTFFixtureType;

	/**
	 * The Description of a GDTF, corresponds to the description.xml of the GDTF.
	 *
	 * The descritpion and all related members in this module shall follow the currently implemented GDTF standard.
	 * See DMXGDTFVersion for details.
	 */
	class DMXGDTF_API FDMXGDTFDescription
		: public TSharedFromThis<FDMXGDTFDescription>
	{
	public:
		/** Initializes this node and its chilren from a GDTF Description.xml */
		void InitializeFromDescriptionXml(const TSharedRef<FXmlFile>& DescriptionXml);

		/** Initializes this object from a GDTF fixture type. */
		void InitializeFromFixtureType(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		/** Exports an Xml file from the node tree in this description. Returns the XML file or nullptr if not a valid XML */
		TSharedPtr<FXmlFile> ExportAsXml() const;

		/** Returns the Fixture Type of this GDTF description */
		TSharedPtr<FDMXGDTFFixtureType> GetFixtureType() const { return FixtureType; }

	private:
		/** The Fixture Type Child Node */
		TSharedPtr<FDMXGDTFFixtureType> FixtureType;
	};
}
