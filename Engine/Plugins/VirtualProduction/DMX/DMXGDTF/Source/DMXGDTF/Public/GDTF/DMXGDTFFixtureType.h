// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/Guid.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFAttributeDefinitions;
	class FDMXGDTFDescription;
	class FDMXGDTFDMXMode;
	class FDMXGDTFFTPreset;
	class FDMXGDTFGeometryCollect;
	class FDMXGDTFModel;
	class FDMXGDTFPhysicalDescriptions;
	class FDMXGDTFProtocols;
	class FDMXGDTFRevision;
	class FDMXGDTFWheel;

	/** The FixtureType node is the starting point of the description of the fixture type within the XML file. */
	class DMXGDTF_API FDMXGDTFFixtureType
		: public FDMXGDTFNode
	{
	public:
		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FixtureType"); }
		virtual void Initialize(const FXmlNode& InXmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Name of the fixture type. As it is based on Name. */
		FName Name;

		/** Shortened name of the fixture type. Non detailed version or an abbreviation. Can use any characters or symbols */
		FString ShortName;

		/** Detailed, complete name of the fixture type, can include any characters or extra symbols */
		FString LongName;

		/** Manufacturer of the fixture type. */
		FString Manufacturer;

		/** Description of the fixture type. */
		FString Description;

		/** Unique number of the fixture type */
		FGuid FixtureTypeID;

		/**
		 * Optional. File name without extension containing description of the thumbnail. Use the following as a resource file:
		 * - png file to provide the rasterized picture. Maximum resolution of picture: 1024x1024
		 * - svg file to provide the vector graphic.
		 * - These resource files are located in the root directory of the zip file.
		 */
		FString Thumbnail;

		/** Horizontal offset in pixels from the top left of the viewbox to the insertion point on a label */
		int32 ThumbnailOffsetX = 0;

		/** Vertical offset in pixels from the top left of the viewbox to the insertion point on a label. */
		int32 ThumbnailOffsetY = 0;

		/** GUID of the referenced fixture type */
		FGuid RefFT;

		/**
		 * Describes if it is possible to mount other devices to this device. Value: “Yes”, “No”. Default value: “Yes”
		 *
		 * UE Specific: Using a bool instead of a true/false enum.
		 */
		bool bCanHaveChildren = true;

		/** Defines all Fixture Type Attributes that are used in the fixture type. */
		TSharedPtr<FDMXGDTFAttributeDefinitions> AttributeDefinitions;

		/** 
		 * This section defines all physical or virtual wheels of the device (XML node <Wheels>). As children wheel collect can have a list of a wheels.
		 * 
		 * Note 1: Physical or virtual wheels represent the changes to the light beam within the device. Typically color, gobo, prism, animation, content and others are described by wheels.
		 */
		TArray<TSharedPtr<FDMXGDTFWheel>> Wheels;

		/** Contains additional physical descriptions. */
		TSharedPtr<FDMXGDTFPhysicalDescriptions> PhysicalDescriptions;

		/**
		 * Each device is divided into smaller parts: body, yoke, head and so on. These are called geometries. Each
		 * geometry has a separate model description and a physical description. Model collect contains model
		 * descriptions of the fixture parts. (XML node <Models>).
		 */
		TArray<TSharedPtr<FDMXGDTFModel>> Models;

		/** Describes physically separated parts of the device. */
		TSharedPtr<FDMXGDTFGeometryCollect> GeometryCollect;

		/**
		 * This section is describes all DMX modes of the device. If firmware revisions change a DMX footprint, then such
		 * revisions should be specified as new DMX mode (XML node <DMXModes>). 
		 */
		TArray<TSharedPtr<FDMXGDTFDMXMode>> DMXModes;

		/** This section defines the history of device type (XML node <Revisions>). */
		TArray<TSharedPtr<FDMXGDTFRevision>> Revisions;

		/** Is used to transfer user-defined and fixture type specific presets to other show files. */
		TArray<TSharedPtr<FDMXGDTFFTPreset>> FTPresets;

		/** Is used to specify supported protocols. */
		TSharedPtr<FDMXGDTFProtocols> Protocols;	
	};
}
