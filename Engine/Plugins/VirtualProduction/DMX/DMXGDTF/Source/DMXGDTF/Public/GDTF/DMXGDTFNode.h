// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFVersion.h"
#include "Templates/SharedPointer.h"

class FXmlNode;

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;
	class FDMXGDTFXmlNodeBuilder;

	/** Base class for GDTF nodes. */
	class DMXGDTF_API FDMXGDTFNode
		: public TSharedFromThis<FDMXGDTFNode>
	{
		template <typename NodeType> friend class FDMXGDTFNodeInitializer;
		friend class FDMXGDTFXmlNodeBuilder;

	public:
		virtual ~FDMXGDTFNode() {}

		/** Gets the Xml Tag corresponding to this node */
		virtual const TCHAR* GetXmlTag() const = 0;

		/** Initializes the node from an Xml node. Called after the node was constructed. */
		virtual void Initialize(const FXmlNode& InXmlNode) = 0;

		/** Creates an XML node in the parent node */
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) = 0;

		/** Returns the fixture type this node resides in */
		TWeakPtr<FDMXGDTFFixtureType> GetFixtureType() const { return WeakFixtureType; }

	private:
		/** The fixture type this node resides in */
		TWeakPtr<FDMXGDTFFixtureType> WeakFixtureType;
	};
}
