// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

namespace UE::DMX::GDTF
{
	/** Version of the GDTF standard currently being used by the engine */
	struct DMXGDTF_API FDMXGDTFVersion
	{
		static const uint8 MajorVersion = 1;

		static const uint8 MinorVersion = 2;

		static const FString GetMajorVersionAsString();
		static const FString GetMinorVersionAsString();

		/** Returns the max supported major and minor version as string, in the format of 'Major.Minor', e.g. "1.2" */
		static const FString GetAsString();
	};
}
