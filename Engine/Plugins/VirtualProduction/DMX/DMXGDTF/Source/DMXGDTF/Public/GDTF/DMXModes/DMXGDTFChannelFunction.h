// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "GDTF/DMXModes/DMXGDTFDMXValue.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFAttribute;
	class FDMXGDTFChannelSet;
	class FDMXGDTFColorSpace;
	class FDMXGDTFDMXChannel;
	class FDMXGDTFDMXMode;
	class FDMXGDTFDMXProfile;
	class FDMXGDTFEmitter;
	class FDMXGDTFFilter;
	class FDMXGDTFFixtureType;
	class FDMXGDTFGamut;
	class FDMXGDTFLogicalChannel;
	class FDMXGDTFPhysicalDesriptions;
	class FDMXGDTFSubchannelSet;
	class FDMXGDTFWheel;

	/** The Fixture Type Attribute is assigned to a Channel Function and defines the function of its DMX Range. (XML node <ChannelFunction>). */
	class DMXGDTF_API FDMXGDTFChannelFunction
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFChannelFunction(const TSharedRef<FDMXGDTFLogicalChannel>& InLogicalChannel);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("ChannelFunction"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** Unique name; Default value: Name of attribute and number of channel function. */
		FName Name;

		/** Link to attribute. */
		FString Attribute;

		/**  The manufacturer’s original name of the attribute; Default: empty */
		FString OriginalAttribute;

		/**
		 * Start DMX value; The end DMX value is calculated as a DMXFrom of the next channel function -1 or the maximum value of the DMX channel.
		 * Default value: "0/1".
		 */
		FDMXGDTFDMXValue DMXFrom;

		/** Default DMX value of channel function when activated by the control system. */
		FDMXGDTFDMXValue Default;

		/** Physical start value; Default value : 0 */
		float PhysicalFrom = 0.f;

		/** Physical end value; Default value : 1 */
		float PhysicalTo = 1.f;

		/** Time in seconds to move from min to max of the Channel Function; Default value : 0 */
		float RealFade = 0.f;

		/** Time in seconds to accelerate from stop to maximum velocity; Default value : 0 */
		float RealAcceleration = 0.f;

		/** (Optional) Link to a wheel; Starting point: Wheel Collect */
		FString Wheel;

		/** (Optional) Link to an emitter in the physical description; Starting point: Emitter Collect */
		FString Emitter;

		/** (Optional) Link to a filter in the physical description; Starting point: Filter Collect */
		FString Filter;

		/** (Optional) Link to a color space in the physical description; Starting point: Physical Descriptions Collect */
		FString ColorSpace;

		/** (Optional) Link to a gamut in the physical description; Starting point: Gamut Collect */
		FString Gamut;

		/** (Optional) Link to DMX Channel or Channel Function; Starting point DMX mode. */
		FString ModeMaster;

		/** Only used together with ModeMaster; DMX start value; Default value: 0/1 */
		FDMXGDTFDMXValue ModeFrom = TEXT("0/1");

		/** Only used together with ModeMaster; DMX start value; Default value: 0/1 */
		FDMXGDTFDMXValue ModeTo = TEXT("0/1");

		/** (Optional) Link to DMX Profile; Starting point: DMX Profile Collect */
		FString DMXProfile;

		/** Minimum Physical Value that will be used for the DMX Profile. Default: Value from PhysicalFrom 1 */
		float Min = 0.f;

		/**  Maximum Physical Value that will be used for the DMX Profile. Default: Value from PhysicalTo */
		float Max = 1.f;

		/**
		 * Custom Name that can he used do adress this channel function with
		 *	other command based protocols like OSC. Default: Node Name of the
		 *	Channel function Example : Head_Dimmer.Dimmer.Dimmer
		 */
		FString CustomName;

		/** A list of channel sets */
		TArray<TSharedPtr<FDMXGDTFChannelSet>> ChannelSetArray;

		/** A list of subchannel sets */
		TArray<TSharedPtr<FDMXGDTFSubchannelSet>> SubchannelSetArray;

		/** The outer logical channel */
		const TWeakPtr<FDMXGDTFLogicalChannel> OuterLogicalChannel;

		/** Resolves the linked attribute. Returns the attribute, or nullptr if no attribute is linked */
		TSharedPtr<FDMXGDTFAttribute> ResolveAttribute() const; 

		/** Resolves the linked wheel. Returns the wheel, or nullptr if no wheel is linked */
		TSharedPtr<FDMXGDTFWheel> ResolveWheel() const;

		/** Resolves the linked emitter. Returns the emitter, or nullptr if no emitter is linked */
		TSharedPtr<FDMXGDTFEmitter> ResolveEmitter() const;

		/** Resolves the linked filter. Returns the filter, or nullptr if no filter is linked */
		TSharedPtr<FDMXGDTFFilter> ResolveFilter() const;

		/** Resolves the linked color space. Returns the color space, or nullptr if no color space is linked */
		TSharedPtr<FDMXGDTFColorSpace> ResolveColorSpace() const;

		/** Resolves the linked gamut. Returns the gamut, or nullptr if no gamut is linked */
		TSharedPtr<FDMXGDTFGamut> ResolveGamut() const;

		/**
		 * Resolves the linked mode master. May be either a DMX channel or a DMX channel function.
		 * Returns the mode master, or nullptr if no mode master is linked.
		 */
		void ResolveModeMaster(TSharedPtr<FDMXGDTFDMXChannel>& OutDMXChannel, TSharedPtr<FDMXGDTFChannelFunction>& OutChannelFunction) const;

		/** Same as resolve mode master but using inclusive language */
		void ResolveModePrimary(TSharedPtr<FDMXGDTFDMXChannel>& OutDMXChannel, TSharedPtr<FDMXGDTFChannelFunction>& OutChannelFunction) const;

		/** Resolves the linked DMX profile. Returns the DMX profile, or nullptr if no DMX profile is linked */
		TSharedPtr<FDMXGDTFDMXProfile> ResolveDMXProfile() const;

		/** Parses the default value. Useful to parse legacy GDTFs that store the default in the DMX channel node */
		FDMXGDTFDMXValue ParseDefault(const FString& Value, const FXmlNode* XmlNode) const;
	};
}
