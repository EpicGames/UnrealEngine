// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFChannelRelation.generated.h"

/** Type of the relation; Values: “Multiply”, “Override” */
UENUM()
enum class EDMXGDTFChannelRelationType
{
	Multiply UMETA(DisplayName = "Multiply"),
	Override UMETA(DisplayName = "Override"),

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXGDTFChannelRelationType, EDMXGDTFChannelRelationType::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFChannelFunction;
	class FDMXGDTFDMXChannel;
	class FDMXGDTFDMXMode;

	/** This section defines the relation between the master DMX channel and the following logical channel(XML node <Relation>). */
	class DMXGDTF_API FDMXGDTFChannelRelation
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFChannelRelation(const TSharedRef<FDMXGDTFDMXMode>& InDMXMode);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Relation"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** The unique name of the relation */
		FName Name;

		/** Link to the master DMX channel */
		FString Master;

		/** Link to the following channel function */
		FString Follower;

		/** Type of the relation */
		EDMXGDTFChannelRelationType Type = EDMXGDTFChannelRelationType::Multiply;

		/** The outer DMX mode */
		const TWeakPtr<FDMXGDTFDMXMode> OuterDMXMode;

		/** Resolves the linked master. Returns the master, or nullptr if no master is linked */
		TSharedPtr<FDMXGDTFDMXChannel> ResolveMaster() const;

		/** Resolves the linked follower. Returns the follower, or nullptr if no follower is linked */
		TSharedPtr<FDMXGDTFChannelFunction> ResolveFollower() const;
	};
}
