// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "DMXGDTFDMXValue.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFChannelFunction;

	/** This section defines the channel sets of the channel function (XML node <ChannelSet>). */
	class DMXGDTF_API FDMXGDTFChannelSet
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFChannelSet(const TSharedRef<FDMXGDTFChannelFunction>& InChannelFunction);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("ChannelSet"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** The name of the channel set. Default: Empty */
		FName Name;

		/**
		 * Start DMX value; The end DMX value is calculated as a DMXFrom of the next
		 * channel set — 1 or the maximum value of the current channel function;
		 * Default value : 0 / 1
		 */
		FDMXGDTFDMXValue DMXFrom;

		/** Physical start value */
		float PhysicalFrom = 0.f;

		/** Physical end value */
		float PhysicalTo = 1.f;

		/**
		 * If the channel function has a link to a wheel, a corresponding slot index
		 * needs to be specified.The wheel slot index results from the order of slots of
		 * the wheel which is linked in the channel function. The wheel slot index is
		 * normalized to 1. Size: 4 bytes
		 */
		int32 WheelSlotIndex = INDEX_NONE;

		/** The outer channel function */
		const TWeakPtr<FDMXGDTFChannelFunction> OuterChannelFunction;
	};
}
