// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "GDTF/DMXModes/DMXGDTFDMXValue.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFChannelFunction;
	class FDMXGDTFDMXMode;
	class FDMXGDTFGeometry;
	class FDMXGDTFGeometryReference;
	class FDMXGDTFLogicalChannel;

	/**
	 * This section defines the DMX channel (XML node <DMXChannel>). The name of a DMX channel cannot be
	 * user-defined and must consist of a geometry name and the attribute name of the first logical channel with the
	 * separator "_". In one DMX Mode, this combination needs to be unique.
	 */
	class DMXGDTF_API FDMXGDTFDMXChannel
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFDMXChannel(const TSharedRef<FDMXGDTFDMXMode>& InDMXMode);

		/** Explicit constructors, required to avoid deprecation warnings on Clang with deprecated member Default. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FDMXGDTFDMXChannel(const FDMXGDTFDMXChannel& Node) = delete;
		FDMXGDTFDMXChannel& operator=(const FDMXGDTFDMXChannel& Node) = delete;
		FDMXGDTFDMXChannel(FDMXGDTFDMXChannel&& Node) = delete;
		FDMXGDTFDMXChannel& operator=(FDMXGDTFDMXChannel&& Node) = delete;
		~FDMXGDTFDMXChannel() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("DMXChannel"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/**
		 * Number of the DMXBreak; Default value: 1;
		 * Special value: “Overwrite” – means that this number will be overwritten by Geometry Reference; Size: 4 bytes
		 *
		 * UE Specific: If negative and a related geometry reference exists, it means the value is set to special value "Overwrite".
		 */
		int32 DMXBreak = 1;

		/**
		 * Relative addresses of the current DMX channel from highest to least significant; Size per int: 4 bytes
		 *
		 * UE Specific: Special value "None" is equviavalent to an empty array.
		 */
		TArray<uint32> Offset;

		/**
		 * Link to the channel function that will be activated by default for this DMXChannel.
		 * Default value is the first channel function of the first logical function of this DMX channel.
		 */
		FString InitialFunction;

		/** Highlight value for current channel; Special value : “None”.Default value : “None” */
		FDMXGDTFDMXValue Highlight;

		/**
		 * Name of the geometry the current channel controls.
		 * The Geometry should be the place in the tree of geometries where the function of the DMX Channel (as
		 * defined by ChannelFunction) is located either physically or logically. If the DMX channel doesn’t have a
		 * location, put it in the top level geometry of the geometry tree. Attributes follow a trickle down principle, so
		 * they are inherited from top down
		 */
		FName Geometry;

		/** A list of logical channels */
		TArray<TSharedPtr<FDMXGDTFLogicalChannel>> LogicalChannelArray;

		/** The outer DMX mode */
		const TWeakPtr<FDMXGDTFDMXMode> OuterDMXMode;

		UE_DEPRECATED(5.5, "Deprecated with GDTF 1.1. Instead each channel function can hold its own default. Please refer to DMXGDTFChannelFunction::Default")
		FDMXGDTFDMXValue Default;

		/** Resolves the linked initial function. Returns the initial function, or nullptr if no initial function is linked */
		TSharedPtr<FDMXGDTFChannelFunction> ResolveInitialFunction() const;

		/** 
		 * Resolves the linked geometry. Returns the geometry, or nullptr if no geometry is linked. 
		 * To resolve as geometry references, see ResolveGeometryReferences.
		 * 
		 * Note, GDTFs of older version directly reference a model as geometry. Such models are not considered.
		 */
		TSharedPtr<FDMXGDTFGeometry> ResolveGeometry() const;

		/** Resolves the linked geometry as geometry references. Returns geometry references or an empty array, if no linked geometry references could be found. */
		TArray<TSharedPtr<FDMXGDTFGeometryReference>> ResolveGeometryReferences() const;

	private:
		/** Converts a GDTF string to an array of offsets */
		TArray<uint32> ParseOffset(const FString& GDTFString) const;
	};
}
