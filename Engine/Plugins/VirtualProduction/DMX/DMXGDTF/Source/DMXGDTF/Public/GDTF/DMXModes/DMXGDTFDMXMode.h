// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFChannelFunction;
	class FDMXGDTFChannelRelation;
	class FDMXGDTFDMXChannel;
	class FDMXGDTFFixtureType;
	class FDMXGDTFFTMacro;
	class FDMXGDTFGeometry;

	/** Each DMX mode describes logical control a part of the device in a specific mode (XML node <DMXMode>). */
	class DMXGDTF_API FDMXGDTFDMXMode
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFDMXMode(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("DMXMode"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the DMX mode */
		FName Name;

		/** Description of the DMX mode. */
		FString Description;

		/** The first geometry in the device; Only top level geometries are allowed to be linked	 */
		FName Geometry;

		/** Description of all DMX channels used in the mode */
		TArray<TSharedPtr<FDMXGDTFDMXChannel>> DMXChannels;

		/** Description of relations between channels */
		TArray<TSharedPtr<FDMXGDTFChannelRelation>> Relations;

		/** Is used to describe macros of the manufacturer. */
		TArray<TSharedPtr<FDMXGDTFFTMacro>> FTMacros;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;

		/** Resolves the Geometry for this mode */
		TSharedPtr<FDMXGDTFGeometry> ResolveGeometry() const;

		/** Resolves a DMX channel or channel function */
		void ResolveChannel(const FString& Link, TSharedPtr<FDMXGDTFDMXChannel>& OutDMXChannel, TSharedPtr<FDMXGDTFChannelFunction>& OutChannelFunction) const;
	};
}
