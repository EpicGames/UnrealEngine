// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatform.h"
#include "Templates/SharedPointer.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFDMXChannel;

	/**
	 * Special type to define DMX value where n is the byte count. The byte count can be individually specified without depending on the resolution of the DMX Channel.
	 * By default byte mirroring is used for the conversion. So 255/1 in a 16 bit channel will result in 65535.
	 * You can use the byte shifting operator to use byte shifting for the conversion. So 255/1s in a 16 bit channel will result in 65280.
	*/
	struct DMXGDTF_API FDMXGDTFDMXValue
	{
		FDMXGDTFDMXValue() = default;
		FDMXGDTFDMXValue(const TCHAR* InValue);
		FDMXGDTFDMXValue(const uint32 InValue, const int32 InNumBytes, const bool bInByteMirroring = true);

		/** Gets the DMX value in the resolution of the specified DMX channel. Returns false if the DMX Value has special value "None" */
		bool Get(const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannel, uint32& OutValue) const;
	
		/** Gets the DMX value in the resolution of the specified DMX channel, only valid when not IsSet (checked). */
		uint32 GetChecked(const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannel) const;

		/** Returns the DMX value as string. If the DMX value is not set, returns 0/1. */
		FString AsString() const;

		/** Resets the DMX value */
		void Reset();

		/** Returns true if the DMX Value is set. */
		bool IsSet() const; 

		bool operator==(const FDMXGDTFDMXValue& Other) const { return Value == Other.Value; }
		bool operator!=(const FDMXGDTFDMXValue& Other) const { return Value != Other.Value; }


	private:
		/** Returns the Max value of WordSize */
		uint32 GetMax(uint8 WordSize) const;

		/** The DMX Value */
		uint32 Value = 0;

		/** The number of bytes of this DMX value */
		uint8 NumBytes = 0;

		/** Whether to use byte mirroring or byte shifting when accessing the DMX value */
		bool bByteMirroring = true;
	};
}
