// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFChannelFunction;
	class FDMXGDTFDMXMode;
	class FDMXGDTFMacroDMX;

	/** This section defines a DMX sequence. (XML node <FTMacro>).  */
	class DMXGDTF_API FDMXGDTFFTMacro
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFFTMacro(const TSharedRef<FDMXGDTFDMXMode>& InDMXMode);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FTMacro"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** The unique name of the macro. */
		FName Name;

		/** (Optional) Link to the channel function. */
		FString ChannelFunction;

		/** A list of macros */
		TArray<TSharedPtr<FDMXGDTFMacroDMX>> MacroDMXArray;

		/** The outer DMX mode */
		const TWeakPtr<FDMXGDTFDMXMode> OuterDMXMode;

		/** Resolves the linked channel function. Returns the channel function, or nullptr if no channel function is linked */
		TSharedPtr<FDMXGDTFChannelFunction> ResolveChannelFunction() const;
	};
}
