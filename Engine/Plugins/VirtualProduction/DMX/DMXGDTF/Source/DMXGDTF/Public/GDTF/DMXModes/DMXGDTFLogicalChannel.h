// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFLogicalChannel.generated.h"

/**
 * If snap is enabled, the logical channel will not fade between values.
 * Instead, it will jump directly to the new value. Value: “Yes”, “No”,
 * “On”, “Off”.Default value : “No”
 */
UENUM()
enum class EDMXGDTFLogicalChannelSnap : uint8
{
	Yes UMETA(DisplayName = "Yes"),
	No UMETA(DisplayName = "No"),
	On UMETA(DisplayName = "On"),
	Off UMETA(DisplayName = "Off"),

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXGDTFLogicalChannelSnap, EDMXGDTFLogicalChannelSnap::MaxEnumValue);

/**
 * Defines if all the subordinate channel functions react to a Group
 * Control defined by the control system. Values: “None”, “Grand”,
 * “Group”; Default value : “None”.
 */
UENUM()
enum class EDMXGDTFLogicalChannelMaster : uint8
{
	None UMETA(DisplayName = "None"),
	Grand UMETA(DisplayName = "Grand"),
	Group UMETA(DisplayName = "Group"),

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXGDTFLogicalChannelMaster, EDMXGDTFLogicalChannelMaster::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFAttribute;
	class FDMXGDTFChannelFunction;
	class FDMXGDTFDMXChannel;

	/**
	 * The Fixture Type Attribute is assigned to a LogicalChannel and defines the function of the LogicalChannel. All
	 * logical channels that are children of the same DMX channel are mutually exclusive. In a DMX mode, only one
	 * logical channel with the same attribute can reference the same geometry at a time. The name of a Logical
	 * Channel cannot be user-defined and is equal to the linked attribute name. The XML node of the logical channel
	 * is <LogicalChannel>.
	 */
	class DMXGDTF_API FDMXGDTFLogicalChannel
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFLogicalChannel(const TSharedRef<FDMXGDTFDMXChannel>& InDMXChannel);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("LogicalChannel"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** Link to the attribute; The starting point is the Attribute Collect. */
		FName Attribute;

		/**  If snap is enabled, the logical channel will not fade between values. Instead, it will jump directly to the new value.*/
		EDMXGDTFLogicalChannelSnap Snap = EDMXGDTFLogicalChannelSnap::No;

		/** Defines if all the subordinate channel functions react to a Group Control defined by the control system.*/
		EDMXGDTFLogicalChannelMaster Master = EDMXGDTFLogicalChannelMaster::None;

		/** Minimum fade time for moves in black action. MibFade is defined for the complete DMX range. Default value : 0; Unit: second */
		float MibFade = 0.f;

		/**
		 * Minimum fade time for the subordinate channel functions to change
		 * DMX values by the control system.DMXChangeTimeLimit is defined
		 * for the complete DMX range.Default value : 0; Unit: second
		 */
		float DMXChangeTimeLimit = 0.f;

		/** A list of channel functions */
		TArray<TSharedPtr<FDMXGDTFChannelFunction>> ChannelFunctionArray;

		/** The outer DMX channel */
		const TWeakPtr<FDMXGDTFDMXChannel> OuterDMXChannel;

		/** Resolves the linked attribute. Returns the attribute, or nullptr if no attribute is linked */
		TSharedPtr<FDMXGDTFAttribute> ResolveAttribute() const;

	private:
		/** Converts a string to a snap enum value. Logs invalid string values. */
		EDMXGDTFLogicalChannelSnap ParseSnap(const FString& GDTFString) const;

		/** Converts a string to a master enum value. Logs invalid string values. */
		EDMXGDTFLogicalChannelMaster ParseMaster(const FString& GDTFString) const;
	};
}
