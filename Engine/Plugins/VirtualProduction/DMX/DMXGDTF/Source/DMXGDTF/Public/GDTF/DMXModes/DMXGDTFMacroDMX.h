// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFTMacro;
	class FDMXGDTFMacroDMXStep;

	/** This section defines the sequence of DMX values which are sent by a control system. (XML node <MacroDMX>). */
	class DMXGDTF_API FDMXGDTFMacroDMX
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFMacroDMX(const TSharedRef<FDMXGDTFFTMacro>& InFTMacro);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("MacroDMX"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** A list of macro steps */
		TArray<TSharedPtr<FDMXGDTFMacroDMXStep>> MacroDMXStepArray;

		/** The outer macro */
		const TWeakPtr<FDMXGDTFFTMacro> OuterFTMacro;
	};
}
