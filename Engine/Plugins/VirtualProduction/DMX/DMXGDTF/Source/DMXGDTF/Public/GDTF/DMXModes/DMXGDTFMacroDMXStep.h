// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFMacroDMX;
	class FDMXGDTFMacroDMXValue;

	/** This section defines a DMX step (XML node <MacroDMXStep>). */
	class DMXGDTF_API FDMXGDTFMacroDMXStep
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFMacroDMXStep(const TSharedRef<FDMXGDTFMacroDMX>& InMacroDMX);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("MacroDMXStep"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** Duration of a step; Default value : 1; Unit: seconds. */
		float Duration = 1.f;

		/** A list of macro dmx values */
		TArray<TSharedPtr<FDMXGDTFMacroDMXValue>> MacroDMXValueArray;

		/** The outer macro DMX */
		const TWeakPtr<FDMXGDTFMacroDMX> OuterMacroDMX;
	};
}
