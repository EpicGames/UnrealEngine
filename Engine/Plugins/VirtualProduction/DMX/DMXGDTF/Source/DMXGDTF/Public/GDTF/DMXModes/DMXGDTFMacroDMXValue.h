// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFDMXChannel;
	class FDMXGDTFMacroDMXStep;

	/** This section defines the value for DMX channel (XML node <DMXValue>). */
	class DMXGDTF_API FDMXGDTFMacroDMXValue
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFMacroDMXValue(const TSharedRef<FDMXGDTFMacroDMXStep>& InMacroDMXStep);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("DMXValue"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** Value of the DMX channel */
		FString DMXValue;

		/** Link to the channel */
		FString DMXChannel;

		/** The outer macro DMX step */
		TWeakPtr<FDMXGDTFMacroDMXStep> OuterMacroDMXStep;

		/** Resolves the linked DMX channel. Returns the DMX channel, or nullptr if no DMX channel is linked */
		TSharedPtr<FDMXGDTFDMXChannel> ResolveDMXChannel() const;
	};
}
