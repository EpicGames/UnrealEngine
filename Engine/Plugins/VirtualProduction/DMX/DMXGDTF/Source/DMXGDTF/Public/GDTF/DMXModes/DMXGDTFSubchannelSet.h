// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "DMXGDTFDMXValue.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFChannelFunction;
	class FDMXGDTFDMXProfile;
	class FDMXGDTFSubphysicalUnit;

	/** This section defines the sub channel sets of the channel function (XML node <ChannelFunction>). */
	class DMXGDTF_API FDMXGDTFSubchannelSet
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFSubchannelSet(const TSharedRef<FDMXGDTFChannelFunction>& InChannelFunction);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("SubchannelSet"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The name of the sub channel set. Default: Empty */
		FName Name;

		/** Physical start value */
		float PhysicalFrom = 0.f;

		/** Physical end value */
		float PhysicalTo = 1.f;

		/** (Optional) Link to the sub physical unit; Starting Point: Attribute */
		FString SubphyiscalUnit;

		/** (Optional) Link to the DMX Profile; Starting Point: DMX Profile Collect */
		FString DMXProfile;

		/** The outer channel function */
		const TWeakPtr<FDMXGDTFChannelFunction> OuterChannelFunction;

		/** Resolves the linked subphysical unit. Returns the subphysical unit, or nullptr if no subphysical unit is linked */
		TSharedPtr<FDMXGDTFSubphysicalUnit> ResolveSubphysicalUnit() const;

		/** Resolves the linked DMX profile. Returns the DMX profile, or nullptr if no DMX profile is linked */
		TSharedPtr<FDMXGDTFDMXProfile> ResolveDMXProfile() const;
	};
}
