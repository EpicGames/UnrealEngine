// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;

	/** This section defines fixture type specific presets. It currently does not have any XML attributes (XML node FTPresets>). */
	class DMXGDTF_API FDMXGDTFFTPreset
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFFTPreset(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FTPreset"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;
	};
}
