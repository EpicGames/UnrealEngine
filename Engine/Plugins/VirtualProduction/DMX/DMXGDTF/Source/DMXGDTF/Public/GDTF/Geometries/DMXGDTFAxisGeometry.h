// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFGeometryCollectBase;

	/** This type of geometry defines device parts with a rotation axis (XML node <Axis>).  */
	class DMXGDTF_API FDMXGDTFAxisGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFAxisGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Axis"); }
		//~ End FDMXGDTFNode interface
	};
}
