// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFBeamGeometry.generated.h"

/** 
 * Defines type of the light source; The currently defined types are: 
 * Discharge, Tungsten, Halogen, LED; Default value “Discharge”
 */
UENUM()
enum class EDMXGDTFBeamGeometryLampType : uint8
{
	Discharge UMETA(DisplayName = "Discharge"),
	Tungsten UMETA(DisplayName = "Tungsten"),
	Halogen UMETA(DisplayName = "Halogen"),
	LED UMETA(DisplayName = "LED"),
	
	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXGDTFBeamGeometryLampType, EDMXGDTFBeamGeometryLampType::MaxEnumValue);

/**
 * Beam Type; Specified values: “Wash”, “Spot”, “None”, “Rectangle”; “PC”, 
 * “Fresnel”, “Glow”. Default value “Wash”
 * 
 *  * The <BeamType> describes how the Beam will be rendered.
 * — “Wash”, “Fresnel”, 
 * - “PC” — A conical beam with soft edges and softened field projection.
 * — “Spot” — A conical beam with hard edges.
 * — “Rectangle” — A pyramid-shaped beam with hard edges.
 * — “None”, “Glow” — No beam will be drawn, only the geometry will emit light itself.
 */
UENUM()
enum class EDMXGDTFBeamGeometryBeamType : uint8
{
	Wash UMETA(DisplayName = "Wash"),
	Spot UMETA(DisplayName = "Spot"),
	None UMETA(DisplayName = "None"),
	Rectangle UMETA(DisplayName = "Rectangle"),
	PC UMETA(DisplayName = "PC"),
	Fresnel UMETA(DisplayName = "Fresnel"),
	Glow UMETA(DisplayName = "Glow"),
	
	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXGDTFBeamGeometryBeamType, EDMXGDTFBeamGeometryBeamType::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFEmitter;

	/**
	 * It is a basic geometry type without specification (XML node <Geometry>).
	 *
	 * The beam geometry emits its light into negative Z direction (and Y-up)
	*/
	class DMXGDTF_API FDMXGDTFBeamGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFBeamGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Beam"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** Defines type of the light source */
		EDMXGDTFBeamGeometryLampType LampType = EDMXGDTFBeamGeometryLampType::Discharge;

		/** Power consumption; Default value: 1 000; Unit: Watt */
		float PowerConsumption = 1000.f;

		/** Intensity of all the represented light emitters; Default value: 10 000; Unit: lumen */
		float LuminousFlux = 1000.f;

		/** Color temperature; Default value: 6 000; Unit: kelvin */
		float ColorTemperature = 1000.f;

		/** Beam angle; Default value: 25,0; Unit: degree */
		float BeamAngle = 25.f;

		/** Field angle; Default value: 25,0; Unit: degree */
		float FieldAngle = 25.f;

		/** Throw Ratio of the lens for BeamType Rectangle; Default value: 1; Unit: None */
		float ThrowRatio = 1.f;

		/** Ratio from Width to Height of the Rectangle Type Beam; Default value: 1.7777; Unit: None */
		float RectangleRatio = 1.7777f;

		/** Beam radius on starting point. Default value: 0,05; Unit: meter. */
		float BeamRadius = 0.05f;

		/**
		 * Beam Type
		 * The <BeamType> describes how the Beam will be rendered.
		 * — “Wash”, “Fresnel”,
		 * - “PC” — A conical beam with soft edges and softened field projection.
		 * — “Spot” — A conical beam with hard edges.
		 * — “Rectangle” — A pyramid-shaped beam with hard edges.
		 * — “None”, “Glow” — No beam will be drawn, only the geometry will emit light itself.
		 */
		EDMXGDTFBeamGeometryBeamType BeamType = EDMXGDTFBeamGeometryBeamType::Wash;

		/**
		 * The CRI according to ANSI/IES TM-30 is a quantitative measure of the
		 * ability of the light source showing the object color naturally as it does as
		 * daylight reference. Size 1 byte. Default value 100
		 */
		uint8 ColorRenderingIndex = 100;

		/** 
		 * (Optional) Link to emitter in the physical description; use this to define the 
		 * white light source of a subtractive color mixing system. Starting point: 
		 * Emitter Collect; Default spectrum is a Black-Body with the defined 
		 * ColorTemperature.
		 */
		FString EmitterSpectrum;

		/** Resolves the linked emitter spectrum. Returns the emitter spectrum, or nullptr if no emitter spectrum is linked */
		TSharedPtr<FDMXGDTFEmitter> ResolveEmitterSpectrum() const;
	};
}
