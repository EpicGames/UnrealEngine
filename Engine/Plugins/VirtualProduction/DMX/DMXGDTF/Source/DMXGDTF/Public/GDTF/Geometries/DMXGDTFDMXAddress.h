// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::DMX::GDTF
{
	/** The container for DMX Addresses */
	struct FDMXGDTFDMXAddress
	{
		FDMXGDTFDMXAddress() = default;

		FDMXGDTFDMXAddress(int64 InAbsoluteAddress)
			: AbsoluteAddress(InAbsoluteAddress)
		{}

		FDMXGDTFDMXAddress(int32 InUniverse, int32 InChannel)
		{
			AbsoluteAddress = InUniverse * UniverseSize + InChannel - 1;
		}

		int32 GetUniverse() const { return AbsoluteAddress / UniverseSize; }

		int32 GetChannel() const { return AbsoluteAddress % UniverseSize; }

		int64 AbsoluteAddress = 512;

		static const uint16 UniverseSize = 512;
	};
}
