// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe a self-emitting surface which is used to display visual media (XML node <Display>) */
	class DMXGDTF_API FDMXGDTFDisplayGeometry
		: public FDMXGDTFGeometry
	{
	public:		
		FDMXGDTFDisplayGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Display"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** Name of the mapped texture in Model file that will be swapped out for the media resource */
		FString Texture;
	};
}
