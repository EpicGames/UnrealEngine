// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	/** This type of geometry defines device parts with a beam filter (XML node <FilterBeam>). */
	class DMXGDTF_API FDMXGDTFFilterBeamGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFFilterBeamGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FilterBeam"); }
		//~ End FDMXGDTFNode interface
	};
}
