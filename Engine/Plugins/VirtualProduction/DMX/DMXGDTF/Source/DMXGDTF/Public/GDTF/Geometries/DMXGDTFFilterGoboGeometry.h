// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe device parts which have gobo wheels (XML node <FilterGobo>). */
	class DMXGDTF_API FDMXGDTFFilterGoboGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFFilterGoboGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FilterGobo"); }
		//~ End FDMXGDTFNode interface
	};
}
