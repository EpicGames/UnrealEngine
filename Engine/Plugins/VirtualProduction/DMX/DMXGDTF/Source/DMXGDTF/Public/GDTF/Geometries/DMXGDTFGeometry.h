// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometryCollectBase.h"
#include "Math/Transform.h" 

namespace UE::DMX::GDTF
{
	class FDMXGDTFModel;

	/**
	 * It is a basic geometry type without specification (XML node <Geometry>).
	 *
	 * UE specific: Base class for all geometry nodes.
	 */
	class DMXGDTF_API FDMXGDTFGeometry
		: public FDMXGDTFGeometryCollectBase
	{
	public:
		FDMXGDTFGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect);

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Geometry"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		//~ Begin DMXGDTFGeometryCollectBase interface
		virtual TSharedPtr<FDMXGDTFGeometry> FindGeometryByName(const TCHAR* InName) const override;
		//~ End DMXGDTFGeometryCollectBase interface

		/** The unique name of geometry. */
		FName Name;

		/** Relative position of geometry; Default value : Identity Matrix */
		FTransform Position;

		/**
		 * (Optional) Link to the corresponding model. The model only replaces
		 * the model of the parent of the referenced geometry. The models of
		 * the children of the referenced geometry are not affected. The starting
		 * point is Models Collect. If model is not set, the model is taken from
		 * the referenced geometry.
		 */
		FString Model;

		/** The outer geometry collect */
		const TWeakPtr<FDMXGDTFGeometryCollectBase> OuterGeometryCollect;

		/** Resolves the linked model. Returns the model, or nullptr if no model is linked */
		TSharedPtr<FDMXGDTFModel> ResolveModel() const;
	};
}
