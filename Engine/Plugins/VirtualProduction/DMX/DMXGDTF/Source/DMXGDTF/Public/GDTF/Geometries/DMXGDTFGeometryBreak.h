// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "GDTF/Geometries/DMXGDTFDMXAddress.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFGeometryReference;

	/** This XML node specifies the DMX offset for the DMX channel of the referenced geometry (XML node <Break>) */
	class DMXGDTF_API FDMXGDTFGeometryBreak
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFGeometryBreak(const TSharedRef<FDMXGDTFGeometryReference>& InGeometryReference);

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Break"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** DMX offset; Default value:1 (Means no offset for the corresponding DMX Channel) */
		FDMXGDTFDMXAddress DMXOffset;

		/** Defines the unique number of the DMX Break for which the Offset is given. Size: 1 byte; Default value 1 */
		uint8 DMXBreak = 1;

		/** The outer geometry reference */
		const TWeakPtr<FDMXGDTFGeometryReference> OuterGeometryReference;
	};
}
