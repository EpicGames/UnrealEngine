// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometryCollectBase.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;

	/**
	 * The physical description of the device parts is defined in the geometry collect. Geometry collect can contain a separate geometry or a tree of geometries.
	 * The geometry collect currently does not have any XML attributes (XML node <Geometries>).
	 */
	class DMXGDTF_API FDMXGDTFGeometryCollect
		: public FDMXGDTFGeometryCollectBase
	{
	public:
		FDMXGDTFGeometryCollect(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Geometries"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;
	};
}
