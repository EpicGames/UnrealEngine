// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "GDTF/DMXGDTFNode.h"
#include <functional>

namespace UE::DMX::GDTF
{
	class FDMXGDTFAxisGeometry;
	class FDMXGDTFBeamGeometry;
	class FDMXGDTFDisplayGeometry;
	class FDMXGDTFFilterBeamGeometry;
	class FDMXGDTFFilterColorGeometry;
	class FDMXGDTFFilterGoboGeometry;
	class FDMXGDTFFilterShaperGeometry;
	class FDMXGDTFGeometry;
	class FDMXGDTFGeometryReference;
	class FDMXGDTFInventoryGeometry;
	class FDMXGDTFLaserGeometry;
	class FDMXGDTFMagnetGeometry;
	class FDMXGDTFMediaServerCameraGeometry;
	class FDMXGDTFMediaServerLayerGeometry;
	class FDMXGDTFMediaServerMasterGeometry;
	class FDMXGDTFStructureGeometry;
	class FDMXGDTFSupportGeometry;
	class FDMXGDTFWiringObjectGeometry;
	class IDMXGDTFGeometryNodeInterface;

	/** UE specific. Bases class for all classes that have a geometry collect. */
	class DMXGDTF_API FDMXGDTFGeometryCollectBase
		: public FDMXGDTFNode
	{
	public:
		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override;
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Any General Geometry. */
		TArray<TSharedPtr<FDMXGDTFGeometry>> GeometryArray;

		/** Any Geometry with axis. */
		TArray<TSharedPtr<FDMXGDTFAxisGeometry>> AxisArray;

		/** Any Geometry with a beam filter. */
		TArray<TSharedPtr<FDMXGDTFFilterBeamGeometry>> FilterBeamArray;

		/** Any Geometry with color filter. */
		TArray<TSharedPtr<FDMXGDTFFilterColorGeometry>> FilterColorArray;

		/** Any Geometry with gobo. */
		TArray<TSharedPtr<FDMXGDTFFilterGoboGeometry>> FilterGoboArray;

		/** Any Geometry with shaper. */
		TArray<TSharedPtr<FDMXGDTFFilterShaperGeometry>> FilterShaperArray;

		/** ny Geometry that describes a light output to project. */
		TArray<TSharedPtr<FDMXGDTFBeamGeometry>> BeamArray;

		/** Any Geometry that describes a media representation layer of a media device. */
		TArray<TSharedPtr<FDMXGDTFMediaServerLayerGeometry>> MediaServerLayerArray;

		/** Any Geometry that describes a camera or output layer of a media device. */
		TArray<TSharedPtr<FDMXGDTFMediaServerCameraGeometry>> MediaServerCameraArray;

		/** Any Geometry that describes a master control layer of a media device. */
		TArray<TSharedPtr<FDMXGDTFMediaServerMasterGeometry>> MediaServerMasterArray;

		/** Any Geometry that describes a surface to display visual media. */
		TArray<TSharedPtr<FDMXGDTFDisplayGeometry>> DisplayArray;

		/** Any Reference to already described geometries. */
		TArray<TSharedPtr<FDMXGDTFGeometryReference>> GeometryReferenceArray;

		/** Any Geometry with a laser light output. */
		TArray<TSharedPtr<FDMXGDTFLaserGeometry>> LaserArray;

		/** Any General Geometry. */
		TArray<TSharedPtr<FDMXGDTFWiringObjectGeometry>> WiringObjectArray;

		/** Any Geometry that describes an additional item that can be used for a fixture (like a rain cover).  */
		TArray<TSharedPtr<FDMXGDTFInventoryGeometry>> InventoryArray;

		/**  Any Geometry that describes the internal framing of an object (like members). */
		TArray<TSharedPtr<FDMXGDTFStructureGeometry>> StructureArray;

		/** Any Geometry that describes a support like a base plate or a hoist. */
		TArray<TSharedPtr<FDMXGDTFSupportGeometry>> SupportArray;

		/** Any Geometry that describes a point where other geometries should be attached */
		TArray<TSharedPtr<FDMXGDTFMagnetGeometry>> MagnetArray;

		/** Returns all child geometries. Does not include self. */
		void GetGeometriesRecursive(TArray<TSharedPtr<FDMXGDTFGeometry>>& OutGeometries, TArray<TSharedPtr<FDMXGDTFGeometryReference>>& OutGeometryReferences);

		/** Finds the geometry by name. */
		virtual TSharedPtr<FDMXGDTFGeometry> FindGeometryByName(const TCHAR* InName) const;

		/** Finds geometry references by name. */
		TSharedPtr<FDMXGDTFGeometryReference> FindGeometryReferenceByName(const TCHAR* InName) const;

		/**  
		 * Resolves a string as a link to a geometry. 
		 * 
		 * The string needs to be formated in the form of "Geometry1.Geometry2.[...].GeometryN" whereas Geometry1 resides in the geometry collect of the fixture type. 
		 */
		template <typename GeometryType>
		TSharedPtr<GeometryType> ResolveGeometryLink(const FString& Link) const
		{
			TArray<FString> LinkArray;
			Link.ParseIntoArray(LinkArray, TEXT("."));

			TSharedPtr<const FDMXGDTFGeometryCollectBase> NextGeometryCollect = StaticCastSharedRef<const FDMXGDTFGeometryCollectBase>(SharedThis(this));
			for (const FString& GeometryName : LinkArray)
			{
				if (&LinkArray.Last() == &GeometryName)
				{
					TArray<TSharedPtr<GeometryType>> FinalGeometryArray;
					NextGeometryCollect->GetGeometriesOfType<GeometryType>(FinalGeometryArray);
					const TSharedPtr<GeometryType>* GeometryPtr = Algo::FindBy(FinalGeometryArray, GeometryName, &GeometryType::Name);
					if (GeometryPtr)
					{
						return *GeometryPtr;
					}
				}

				const TSharedPtr<FDMXGDTFGeometry> Geometry = NextGeometryCollect->FindGeometryByName(*GeometryName);
				if (Geometry.IsValid())
				{
					NextGeometryCollect = StaticCastSharedPtr<const FDMXGDTFGeometryCollectBase>(Geometry);
				}
			}

			return nullptr;
		}

		/** 
		 * Returns all geometries of a specific type in this collect.
		 * 
		 * Example:
		 * TArray<FDMXGDTFBeamGeometry> BeamGeometries;
		 * MyGeometryCollect->GetGeometriesOfType(BeamGeometries); // Returns all beam geometries
		 */
		template <typename GeometryType>
		void GetGeometriesOfType(TArray<TSharedPtr<GeometryType>>& OutArray) const
		{
			if constexpr (std::is_same_v<GeometryType, FDMXGDTFGeometry>)
			{
				OutArray = GeometryArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFAxisGeometry>)
			{
				OutArray = AxisArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFFilterBeamGeometry>)
			{
				OutArray = FilterBeamArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFFilterColorGeometry>)
			{
				OutArray = FilterColorArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFFilterGoboGeometry>)
			{
				OutArray = FilterGoboArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFFilterShaperGeometry>)
			{
				OutArray = FilterShaperArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFBeamGeometry>)
			{
				OutArray = BeamArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFMediaServerLayerGeometry>)
			{
				OutArray = MediaServerLayerArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFMediaServerCameraGeometry>)
			{
				OutArray = MediaServerCameraArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFMediaServerMasterGeometry>)
			{
				OutArray = MediaServerMasterArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFDisplayGeometry>)
			{
				OutArray = DisplayArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFGeometryReference>)
			{
				OutArray = GeometryReferenceArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFLaserGeometry>)
			{
				OutArray = LaserArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFWiringObjectGeometry>)
			{
				OutArray = WiringObjectArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFInventoryGeometry>)
			{
				OutArray = InventoryArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFStructureGeometry>)
			{
				OutArray = StructureArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFSupportGeometry>)
			{
				OutArray = SupportArray;
			}
			else if constexpr (std::is_same_v<GeometryType, FDMXGDTFMagnetGeometry>)
			{
				OutArray = MagnetArray;
			}
			else
			{
				[] <bool SupportedType = false>()
				{
					static_assert(SupportedType, "Unsupported geometry type.");
				}();
			}
		}
	};
}
