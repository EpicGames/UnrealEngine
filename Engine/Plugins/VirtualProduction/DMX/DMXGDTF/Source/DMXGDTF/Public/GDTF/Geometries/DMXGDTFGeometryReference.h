// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

#include "Math/Transform.h" 

namespace UE::DMX::GDTF
{
	class FDMXGDTFGeometryBreak;
	class FDMXGDTFGeometry;
	class FDMXGDTFGeometryCollectBase;
	class FDMXGDTFModel;

	/**
	 * The Geometry Type Reference is used to describe multiple instances of the same geometry. Example: LED panel with multiple pixels. (XML node <GeometryReference>).
	 */
	class DMXGDTF_API FDMXGDTFGeometryReference
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFGeometryReference(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect);
		
		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("GeometryReference"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** The unique name of geometry. */
		FName Name;

		/** Relative position of geometry; Default value : Identity Matrix */
		FTransform Position;

		/** The referenced geometry. Only top level geometries are allowed to be referenced */
		FName Geometry;

		/**
		 * (Optional) Link to the corresponding model. The model only replaces
		 * the model of the parent of the referenced geometry. The models of
		 * the children of the referenced geometry are not affected. The starting
		 * point is Models Collect. If model is not set, the model is taken from
		 * the referenced geometry.
		 */
		FName Model;

		/**
		 * As children, the Geometry Type Reference has a list of a breaks. The count of the children depends on the
		 * number of different breaks in the DMX channels of the referenced geometry. If the referenced geometry, for
		 * example, has DMX channels with DMX break 2 and 4, the geometry reference has to have 2 children. The first
		 * child with DMX offset for DMX break 2 and the second child for DMX break 4. If one or more of the DMX
		 * channels of the referenced geometry have the special value “Overwrite” as a DMX break, the DMX break for
		 * those channels and the DMX offsets need to be defined
		 */
		TArray<TSharedPtr<FDMXGDTFGeometryBreak>> BreakArray;

		/** The outer geometry collect */
		const TWeakPtr<FDMXGDTFGeometryCollectBase> OuterGeometryCollect;

		/** Resolves the linked geometry. Returns the geometry, or nullptr if no geometry is linked */
		TSharedPtr<FDMXGDTFGeometry> ResolveGeometry() const;

		/** Resolves the linked model. Returns the model, or nullptr if no model is linked */
		TSharedPtr<FDMXGDTFModel> ResolveModel() const;
	};
}
