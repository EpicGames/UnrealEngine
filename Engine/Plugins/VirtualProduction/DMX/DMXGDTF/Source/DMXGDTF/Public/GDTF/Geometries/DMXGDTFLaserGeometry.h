// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFLaserGeometry.generated.h"

/** Color type of a Laser Geometry. The currently defined unit values are: “RGB”, “SingleWaveLength”, Default: RGB */
UENUM()
enum class EDMXLaserGeometryColorType
{
	RGB,
	SingleWaveLength,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXLaserGeometryColorType, EDMXLaserGeometryColorType::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFEmitter;
	class FDMXGDTFLaserProtocol;

	/** This type of geometry is used to describe the position of a laser’s light output (XML node <Laser>) */
	class DMXGDTF_API FDMXGDTFLaserGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFLaserGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Laser"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** Color type of a Laser */
		EDMXLaserGeometryColorType ColorType = EDMXLaserGeometryColorType::RGB;

		/** Required if ColorType is “SingleWaveLength”; Unit:nm (nanometers) */
		float Color = 0.f;

		/** Output Strength of the Laser; Unit: Watt */
		float OutputStrength = 0.f;

		/** (Optional) link to the emitter group. The starting point is the Emitter Collect. */
		FString Emitter;

		/** Beam diameter where it leaves the projector; Unit: meter */
		float BeamDiameter = 0.f;

		/** Minimum beam divergence; Unit: mrad (milliradian) */
		float BeamDivergenceMin = 0.f;

		/** Maximum beam divergence; Unit: mrad (milliradian) */
		float BeamDivergenceMax = 0.f;

		/** Possible Total Scan Angle Pan of the beam. Assumes symmetrical output; Unit: Degree */
		float ScanAnglePan = 0.f;

		/** Possible Total Scan Angle Tilt of the beam. Assumes symmetrical output; Unit: Degree */
		float ScanAngleTilt = 0.f;

		/** Speed of the beam; Unit: kilo point per second */
		float ScanSpeed = 0.f;

		/** A list of protocols supported by the laser */
		TArray<TSharedPtr<FDMXGDTFLaserProtocol>> ProtocolArray;

		/** Resolves the linked emitter. Returns the emitter, or nullptr if no emitter is linked */
		TSharedPtr<FDMXGDTFEmitter> ResolveEmitter() const;
	};
}
