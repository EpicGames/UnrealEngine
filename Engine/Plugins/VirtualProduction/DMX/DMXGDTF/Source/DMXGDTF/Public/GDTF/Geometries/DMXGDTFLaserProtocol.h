// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFLaserGeometry;

	/** This XML node specifies the protocol for a Laser (XML node <Protocol>) */
	class DMXGDTF_API FDMXGDTFLaserProtocol
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFLaserProtocol(const TSharedRef<FDMXGDTFLaserGeometry>& InLaserGeometry);

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Protocol"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** Name of the protocol */
		FString Name;

		/** The outer laser geometry */
		const TWeakPtr<FDMXGDTFLaserGeometry> OuterLaserGeometry;
	};
}
