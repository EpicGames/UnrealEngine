// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe a magnet, a point where other geometries should be attached (XML node <Magnet>). */
	class DMXGDTF_API FDMXGDTFMagnetGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFMagnetGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Magnet"); }
		//~ End FDMXGDTFNode interface
	};
}