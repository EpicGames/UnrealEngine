// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe the layer of a media device that is used for representation of media files (XML node <MediaServerLayer>). */
	class DMXGDTF_API FDMXGDTFMediaServerLayerGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFMediaServerLayerGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("MediaServerLayer"); }
		//~ End FDMXGDTFNode interface
	};
}
