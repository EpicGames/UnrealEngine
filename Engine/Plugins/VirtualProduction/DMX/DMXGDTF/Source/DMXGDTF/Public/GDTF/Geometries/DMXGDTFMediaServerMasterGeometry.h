// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe the master control of one or several media devices (XML node <MediaServerMaster>). */
	class DMXGDTF_API FDMXGDTFMediaServerMasterGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFMediaServerMasterGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("MediaServerMaster"); }
		//~ End FDMXGDTFNode interface
	};
}