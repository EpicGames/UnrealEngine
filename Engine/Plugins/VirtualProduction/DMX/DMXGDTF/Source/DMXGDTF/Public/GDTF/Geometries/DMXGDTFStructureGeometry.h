// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFStructureGeometry.generated.h"


/** The type of structure. Defined values are “CenterLineBased”, “Detail”. */
UENUM()
enum class EDMXGDTFStructureGeometryType : uint8
{
	CenterLineBased,
	Detail,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFStructureGeometryType, EDMXGDTFStructureGeometryType::MaxEnumValue);

/** The type of cross section. Defined values are “TrussFramework”, “Tube”. */
UENUM()
enum class EDMXGDTFStructureGeometryCrossSectionType : uint8
{
	TrussFramework,
	Tube,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFStructureGeometryCrossSectionType, EDMXGDTFStructureGeometryCrossSectionType::MaxEnumValue);

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe a structure (XML node <Structure>). */
	class DMXGDTF_API FDMXGDTFStructureGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFStructureGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Structure"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** The linked geometry */
		FString LinkedGeometry;

		/** The type of structure. */
		EDMXGDTFStructureGeometryType StructureType = EDMXGDTFStructureGeometryType::CenterLineBased;

		/** The type of cross section. */
		EDMXGDTFStructureGeometryCrossSectionType CrossSectionType = EDMXGDTFStructureGeometryCrossSectionType::TrussFramework;

		/** The height of the cross section. Only for Tubes. Unit: meter */
		float CrossSectionHeight = 0.f;

		/** The thickness of the wall of the cross section. Only for Tubes. Unit: meter. */
		float CrossSectionWallThickness = 0.f;

		/** The name of the truss cross section. Only for Trusses. */
		FString TrussCrossSection;

		/** Resolves the linked geometry. Returns the geometry, or nullptr if no geometry is linked */
		TSharedPtr<FDMXGDTFGeometry> ResolveLinkedGeometry() const;
	};
}
