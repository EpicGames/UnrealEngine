// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFSupportGeometry.generated.h"

/** The type of support. Defined values are “Rope”, “GroundSupport”. */
UENUM()
enum class EDMXGDTFGeometrySupportType : uint8
{
	Rope,
	GroundSupport,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFGeometrySupportType, EDMXGDTFGeometrySupportType::MaxEnumValue);

namespace UE::DMX::GDTF
{
	/** This type of geometry is used to describe a support (XML node <Support>).  */
	class DMXGDTF_API FDMXGDTFSupportGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFSupportGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Support"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** The type of support */
		EDMXGDTFGeometrySupportType SupportType = EDMXGDTFGeometrySupportType::Rope;

		/** The name of the rope cross section. Only for Ropes. */
		FString RopeCrossSection;

		/** The Offset of the rope from bottom to top. Only for Ropes. Unit: meter */
		FVector RopeOffset = FVector::ZeroVector;

		/** The allowable force on the X-Axis applied to the object according to the Eurocode. Unit: N. */
		float CapacityX = 0.f;

		/** The allowable force on the Y-Axis applied to the object according to the Eurocode. Unit: N. */
		float CapacityY = 0.f;

		/** The allowable force on the Z-Axis applied to the object according to the Eurocode. Unit: N. */
		float CapacityZ = 0.f;

		/** The allowable moment around the X-Axis applied to the object according to the Eurocode. Unit: N/m. */
		float CapacityXX = 0.f;

		/** The allowable moment around the Y-Axis applied to the object according to the Eurocode. Unit: N/m. */
		float CapacityYY = 0.f;

		/** The allowable moment around the Z-Axis applied to the object according to the Eurocode. Unit: N/m. */
		float CapacityZZ = 0.f;

		/** The compression ratio for this support along the X-Axis. Unit N/m. Only for Ground Supports. */
		float ResistanceX = 0.f;

		/** The compression ratio for this support along the Y-Axis. Unit N/m. Only for Ground Supports. */
		float ResistanceY = 0.f;

		/** The compression ratio for this support along the Z-Axis. Unit N/m. Only for Ground Supports. */
		float ResistanceZ = 0.f;

		/** The compression ratio for this support around the X-Axis. Unit N/m. Only for Ground Supports. */
		float ResistanceXX = 0.f;

		/** The compression ratio for this support around the Y-Axis. Unit N/m. Only for Ground Supports. */
		float ResistanceYY = 0.f;

		/** The compression ratio for this support around the Z-Axis. Unit N/m. Only for Ground Supports. */
		float ResistanceZZ = 0.f;
	};
}
