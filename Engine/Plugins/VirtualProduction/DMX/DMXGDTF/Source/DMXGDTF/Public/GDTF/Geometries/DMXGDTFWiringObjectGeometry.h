// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/Geometries/DMXGDTFGeometry.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFWiringObjectGeometry.generated.h"

/**
 * The type of the electrical component used. Defined values are
 * “Input”, “Output”, “PowerSource”, “Consumer”, “Fuse”,
 * “NetworkProvider”, “NetworkInput”, “NetworkOutput”,
 * “NetworkInOut”.
 */
UENUM()
enum class EDMXWiringObjectGeometryComponentType : uint8
{
	Input,
	Output,
	PowerSource,
	Consumer,
	Fuse,
	NetworkProvider,
	NetworkInput,
	NetworkOutput,
	NetworkInOut,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXWiringObjectGeometryComponentType, EDMXWiringObjectGeometryComponentType::MaxEnumValue);

/** Fuse Rating.Defined values are “B”, “C”, “D”, “K”, “Z”. */
UENUM()
enum class EDMXWiringObjectGeometryFuseRating : uint8
{
	B,
	C,
	D,
	K,
	Z,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXWiringObjectGeometryFuseRating, EDMXWiringObjectGeometryFuseRating::MaxEnumValue);

/** Where the pins are placed on the object.Defined values are “Left”, “Right”, “Top”, “Bottom”. */
UENUM()
enum class EDMXWiringObjectGeometryOrientation : uint8
{
	Left,
	Right,
	Top,
	Bottom,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EDMXWiringObjectGeometryOrientation, EDMXWiringObjectGeometryOrientation::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class DMXGDTF_API FDMXGDTFWiringObjectPinPatch;

	/** This type of geometry is used to describe an electrical device that can be wired (XML node <WiringObject>). */
	class DMXGDTF_API FDMXGDTFWiringObjectGeometry
		: public FDMXGDTFGeometry
	{
	public:
		FDMXGDTFWiringObjectGeometry(const TSharedRef<FDMXGDTFGeometryCollectBase>& InGeometryCollect)
			: FDMXGDTFGeometry(InGeometryCollect)
		{};

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("WiringObject"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** Link to the corresponding model. */
		FString Model;

		/**
		 * The type of the connector.Find a list of predefined types in
		 * Annex D.This is not applicable for Component Types Fuses. Custom
		 * type of connector can also be defined, for example “Loose End”.
		 */
		FName ConnectorType;

		/** Relative position of geometry */
		FTransform Matrix = FTransform::Identity;

		/** The type of the electrical component used */
		EDMXWiringObjectGeometryComponentType ComponentType = EDMXWiringObjectGeometryComponentType::Input;

		/**
		 * The type of the signal used.Predefinded values are “Power”,
		 * “DMX512”, “Protocol”, “AES”, “AnalogVideo”, “AnalogAudio”.When
		 * you have a custom protocol, you can add it here.
		 */
		FString SignalType;

		/** The number of available pins of the connector type to connect internal wiring to it. */
		int32 PinCount = 0;

		/** The electrical consumption in Watts.Only for Consumers.Unit: Watt. */
		float ElectricalPayLoad = 0.f;

		/** The voltage range’s maximum value.Only for Consumers.Unit: volt. */
		float VoltageRangeMax = 0.f;

		/** The voltage range’s minimum value.Only for Consumers.Unit: volt. */
		float VoltageRangeMin = 0.f;

		/** The Frequency range’s maximum value.Only for Consumers.Unit: hertz. */
		float FrequencyRangeMax = 0.f;

		/** The Frequency range’s minimum value.Only for Consumers.Unit: hertz. */
		float FrequencyRangeMin = 0.f;

		/** The maximum electrical payload that this power source can handle. Only for Power Sources. Unit: voltampere.  */
		float MaxPayLoad = 0.f;

		/** he voltage output that this power source can handle. Only for Power Sources.Unit : volt. */
		float Voltage = 0.f;

		/**
		 * The layer of the Signal Type. In one device, all wiring geometry that
		 * use the same Signal Layers are connected.Special value 0 :
		 * Connected to all geometries.
		 */
		int32 SignalLayer = 0;

		/** The Power Factor of the device. Only for consumers. */
		float CosPhi = 0.f;

		/** The fuse value.Only for fuses. Unit: ampere.  */
		float FuseCurrent = 0.f;

		/** Fuse Rating */
		EDMXWiringObjectGeometryFuseRating FuseRating = EDMXWiringObjectGeometryFuseRating::B;

		/** Where the pins are placed on the object */
		EDMXWiringObjectGeometryOrientation Orientation = EDMXWiringObjectGeometryOrientation::Left;

		/** Name of the group to which this wiring object belong */
		FString WireGroup;

		/** The wiring object has pin patch children. */
		TArray<TSharedPtr<FDMXGDTFWiringObjectPinPatch>> PinPatchArray;

		/** Resolves the linked model. Returns the model, or nullptr if no model is linked */
		TSharedPtr<FDMXGDTFModel> ResolveModel() const;
	};
}
