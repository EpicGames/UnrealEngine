// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFWiringObjectGeometry;

	/**
	 * This node (XML node <PinPatch>) specifies how the different sockets of its parent wiring object are
	 * connected to the pins of other wiring objects
	 */
	class DMXGDTF_API FDMXGDTFWiringObjectPinPatch
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFWiringObjectPinPatch(const TSharedRef<FDMXGDTFWiringObjectGeometry>& InWiringObjectGeometry);

		//~ Begin DMXGDTFGeneralGeometryNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("PinPatch"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End DMXGDTFGeneralGeometryNode interface

		/** Link to the wiring object connected through this pin patch.  */
		FString ToWiringObject;

		/** The pin number used by the parent wiring object to connect to the targeted wiring object “ToWiringObject”. */
		uint32 FromPin = 0;

		/** The pin number used by the targeted wiring object “ToWiringObject” to connect to the parent wiring object */
		uint32 ToPin = 0;

		/** The outer wiring object geometry */
		const TWeakPtr<FDMXGDTFWiringObjectGeometry> OuterWiringObjectGeometry;

		/** Resolves the linked to wiring object. Returns the to wiring object, or nullptr if no to wiring object is linked */
		TSharedPtr<FDMXGDTFWiringObjectGeometry> ResolveToWiringObject() const;
	};
}
