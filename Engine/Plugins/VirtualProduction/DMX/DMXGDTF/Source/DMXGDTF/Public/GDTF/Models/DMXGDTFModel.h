// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFModel.generated.h"

/**
 * Type of 3D model; The currently defined values are: “Undefined”,
 * “Cube”, “Cylinder”, “Sphere”, “Base”, “Yoke”, “Head”, “Scanner”,
 * “Conventional”, “Pigtail”, “Base1_1”, “Scanner1_1”,
 * “Conventional1_1”;
 * Default value: “Undefined"
 */
UENUM()
enum class EDMXGDTFModelPrimitiveType : uint8
{
	Undefined UMETA(DisplayName = "Undefined"),
	Cube UMETA(DisplayName = "Cube"),
	Cylinder UMETA(DisplayName = "Cylinder"),
	Sphere UMETA(DisplayName = "Sphere"),
	Base UMETA(DisplayName = "Base"),
	Yoke UMETA(DisplayName = "Yoke"),
	Head UMETA(DisplayName = "Head"),
	Scanner UMETA(DisplayName = "Scanner"),
	Conventional UMETA(DisplayName = "Conventional"),
	Pigtail UMETA(DisplayName = "Pigtail"),
	Base1_1 UMETA(DisplayName = "Base1_1"),
	Scanner1_1 UMETA(DisplayName = "Scanner1_1"),
	Conventional1_1 UMETA(DisplayName = "Conventional1_1"),

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFModelPrimitiveType, EDMXGDTFModelPrimitiveType::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;

	/**
	 * Each device is divided into smaller parts: body, yoke, head and so on. These are called geometries. Each geometry has a separate model description and a physical description.
	 * Model collect contains model descriptions of the fixture parts. The model collect currently does not have any XML attributes (XML node <Models>).
	 */
	class DMXGDTF_API FDMXGDTFModel
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFModel(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Model"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the model. */
		FName Name;

		/** Length. Unit: meter; Default value: 0 */
		float Length = 0.0;

		/** Width. Unit: meter; Default value: 0 */
		float Width = 0.0;

		/** Height. Unit: meter; Default value: 0 */
		float Height = 0.0;

		/** Type of 3D model. Default value: “Undefined */
		EDMXGDTFModelPrimitiveType PrimitiveType = EDMXGDTFModelPrimitiveType::Undefined;

		/** (Optional) File name without extension and without subfolder containing description of the model.  */
		FString File;

		/** Offset in X from the 0, 0 point to the desired insertion point of the top view svg. Unit based on the SVG. Default value: 0 */
		float SVGOffsetX = 0.f;

		/** Offset in Y from the 0, 0 point to the desired insertion point of the top view svg. Unit based on the SVG. Default value: 0 */
		float SVGOffsetY = 0.f;

		/** Offset in X from the 0,0 point to the desired insertion point of the side view svg. Unit based on the SVG. Default value: 0 */
		float SVGSideOffsetX = 0.f;

		/** Offset in Y from the 0,0 point to the desired insertion point of the side view svg. Unit based on the SVG. Default value: 0 */
		float SVGSideOffsetY = 0.f;

		/** Offset in X from the 0,0 point to the desired insertion point of the front view svg. Unit based on the SVG. Default value: 0 */
		float SVGFrontOffsetX = 0.f;

		/** Offset in Y from the 0,0 point to the desired insertion point of the front view svg. Unit based on the SVG. Default value: 0 */
		float SVGFrontOffsetY = 0.f;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;
	};
}
