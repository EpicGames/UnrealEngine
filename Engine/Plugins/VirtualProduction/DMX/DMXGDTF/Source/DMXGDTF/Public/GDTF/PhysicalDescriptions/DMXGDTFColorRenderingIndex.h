// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFColorRenderingIndexGroup;

	/** This section defines the DMX ColorRenderingIndex description (XML node <ColorRenderingIndex>). */
	class DMXGDTF_API FDMXGDTFColorRenderingIndex
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFColorRenderingIndex(const TSharedRef<FDMXGDTFColorRenderingIndexGroup>& InColorRenderingIndexGroup);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("CRI"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** 
		 * Color sample. The defined values are “CES01”, “CES02”, ... “CES99”.Default Value “CES01" 
		 * UE specific: The integer value instead of an enumeration, wehre e.g. 1 corresponds to "CES01".
		 */
		uint8 CES = 1;

		/** The color rendering index for this sample. Default value: 100 */
		uint8 ColorRenderingIndex = 100;

		/** The outer color rendering index group */
		const TWeakPtr<FDMXGDTFColorRenderingIndexGroup> OuterColorRenderingIndexGroup;

	private:
		/** Parses color sample from a GDTF string */
		uint8 ParseCES(const FString& GDTFString) const;
	};
}
