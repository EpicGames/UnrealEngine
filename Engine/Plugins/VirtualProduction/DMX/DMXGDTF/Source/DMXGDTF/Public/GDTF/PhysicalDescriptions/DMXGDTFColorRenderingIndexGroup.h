// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFPhysicalDescriptions;
	class FDMXGDTFColorRenderingIndex;

	/** This section contains CRIs for a single color temperature (XML node <CRIGroup>). */
	class DMXGDTF_API FDMXGDTFColorRenderingIndexGroup
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFColorRenderingIndexGroup(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("CRIGroup"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/**  Color temperature; Default value: 6000; Unit: Kelvin */
		float ColorTemperature = 6000.f; 

		/** As children, the CRIGroup has an optional list of Color Rendering Index. */
		TArray<TSharedPtr<FDMXGDTFColorRenderingIndex>> CRIArray;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;
	};
}
