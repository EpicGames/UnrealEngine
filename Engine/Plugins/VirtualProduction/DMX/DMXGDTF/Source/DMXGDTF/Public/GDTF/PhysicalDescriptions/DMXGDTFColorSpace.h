// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "Misc/EnumRange.h"
#include "UObject/NameTypes.h"

#include "DMXGDTFColorSpace.generated.h"

/**
 * Measurement interpolation to
 *
 * The currently defined unit values are:  "Linear", "Step", "Log"; Default: Linear
 */
UENUM(BlueprintType)
enum class EDMXGDTFColorSpaceMode : uint8
{
	Custom,
	sRGB,
	ProPhoto,
	ANSI,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFColorSpaceMode, EDMXGDTFColorSpaceMode::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFPhysicalDescriptions;

	/** This section defines the description of the ColorSpace (XML node <ColorSpace>). */
	class DMXGDTF_API FDMXGDTFColorSpace
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFColorSpace(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("ColorSpace"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface
		
		/** 
		 * Unique Name of the Color Space. Default Value : "Default".
		 * Note that the name need to be unique for the default colorspace and all color spaces in the AdditionalColorSpaces node. 
		 */
		FName Name = "";

		/** 
		 * Definition of the Color Space that used for the indirect color mixing.
		 * The defined values are "Custom", "sRGB", "ProPhoto" and "ANSI".Default Value : "sRGB" 
		 */
		EDMXGDTFColorSpaceMode Mode = EDMXGDTFColorSpaceMode::sRGB;

		/** (Optional) CIE xyY of the Red Primary; this is used only if the ColorSpace is "Custom". */
		FDMXGDTFColorCIE1931xyY Red;	
		
		/** (Optional) CIE xyY of the Green Primary; this is used only if the ColorSpace is "Custom". */
		FDMXGDTFColorCIE1931xyY Green;

		/** (Optional) CIE xyY of the Blue Primary; this is used only if the ColorSpace is "Custom". */
		FDMXGDTFColorCIE1931xyY Blue;
		
		/** (Optional) CIE xyY of the White Point; this is used only if the ColorSpace is "Custom". */
		FDMXGDTFColorCIE1931xyY WhitePoint;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;
	};
}
