// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFPhysicalDescriptions;
	class FDMXGDTFDMXProfilePoint;

	/** This section defines the DMX profile description (XML node <DMXProfile>). */
	class DMXGDTF_API FDMXGDTFDMXProfile
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFDMXProfile(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("DMXProfile"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Unique name of the DMX profile */
		FName Name;

		/** As children a DMX Profile has a list of point. */
		TArray<TSharedPtr<FDMXGDTFDMXProfilePoint>> PointArray;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;
	};
}
