// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFDMXProfile;

	/** 
	 * This section contains points to define the DMX profile (XML node <Point>). 
	 *
	 * Find the Point with the biggest DMXPercentage below or equal x. If there is none, the output is expected to be 0.
	 * Output(x) = CFC3 * (x - DMXPercent)³ + CFC2 * (x - DMXPercent)² + CFC1 * (x - DMXPercent) + CFC0
	 */
	class DMXGDTF_API FDMXGDTFDMXProfilePoint
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFDMXProfilePoint(const TSharedRef<FDMXGDTFDMXProfile>& InDMXProfile);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Point"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** DMX percentage of the point; Unit: Percentage; Default value : 0 */
		float DMXPercentage = 0.f;

		/** Cubic Function Coefficient for x⁰; Default value : 0 */
		float CFC0 = 0.f;

		/** Cubic Function Coefficient for x; Default value : 0 */
		float CFC1 = 0.f;

		/** Cubic Function Coefficient for x²; Default value : 0 */
		float CFC2 = 0.f;

		/** Cubic Function Coefficient for x³; Default value : 0 */
		float CFC3 = 0.f;

		/** The outer DMX profile */
		const TWeakPtr<FDMXGDTFDMXProfile> OuterDMXProfile;
	};
}
