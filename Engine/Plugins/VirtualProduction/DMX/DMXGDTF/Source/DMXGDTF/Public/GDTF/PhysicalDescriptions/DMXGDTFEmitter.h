// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFEmitterMeasurement;
	class FDMXGDTFPhysicalDescriptions;

	/** This section defines the description of the emitter (XML node <Emitter>). */
	class DMXGDTF_API FDMXGDTFEmitter
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFEmitter(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Emitter"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface
		
		/** Unique Name of the emitter */
		FName Name;

		/** 
		 * Approximate absolute color point if applicable.Omit for non - visible emitters (eg., UV).
		 * For Y give relative value compared to overall output defined in property Luminous Flux of related Beam Geometry (transmissive case). 
		 */
		FDMXGDTFColorCIE1931xyY Color;

		/** Required if color is omitted, otherwise it is optional. Dominant wavelength of the LED. */
		float DominantWaveLength;

		/** (Optional) Manufacturer’s part number of the diode. */
		FString DiodePart;
		
		/** As children, the Emitter has a list of measurements. */
		TArray<TSharedPtr<FDMXGDTFEmitterMeasurement>> Measurements;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;
	};
}
