// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFilterMeasurement;
	class FDMXGDTFPhysicalDescriptions;

	/** This section defines the description of the filter (XML node <Filter>). */
	class DMXGDTF_API FDMXGDTFFilter
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFFilter(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Filter"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Unique Name of the filter */
		FName Name;

		/** 
		 * Approximate absolute color point when this filter is the only item fully inserted into the beam and the fixture is at maximum intensity.
		 * For Y give relative value compared to overall output defined in property Luminous Flux of related Beam Geometry(transmissive case).
		 */
		FDMXGDTFColorCIE1931xyY Color;

		/** As children the Filter has a list of measurements. */
		TArray<TSharedPtr<FDMXGDTFFilterMeasurement>> Measurements;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;
	};
}
