// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFPhysicalDescriptions;

	/** This section defines the color gamut of the fixture (XML node <Gamut>), which is the set of attainable colors by the fixture. */
	class DMXGDTF_API FDMXGDTFGamut
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFGamut(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Gamut"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Unique Name of the Gamut. */
		FName Name;	

		/** Set of points defining the vertice of the gamut's polygon. */
		TArray<FDMXGDTFColorCIE1931xyY>	Points;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;

	private:
		/** Parses points from a GDTF string */
		TArray<FDMXGDTFColorCIE1931xyY> ParsePoints(const FString& GDTFString) const;
	};
}
