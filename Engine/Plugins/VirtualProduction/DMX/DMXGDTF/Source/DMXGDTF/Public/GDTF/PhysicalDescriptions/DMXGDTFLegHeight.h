// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFProperties;

	/** This section defines the height of the legs (XML node <LegHeight>).  */
	class DMXGDTF_API FDMXGDTFLegHeight
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFLegHeight(const TSharedRef<FDMXGDTFProperties>& InProperties);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("LegHeight"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Defines height of the legs - distance between the floor and the bottom base plate. Unit: meter. Default value: 0 */
		float Value = 0.f;	
	
		/** The outer properties */
		const TWeakPtr<FDMXGDTFProperties> OuterProperties;
	};
}
