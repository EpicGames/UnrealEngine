// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/EnumRange.h"

#include "DMXGDTFMeasurement.generated.h"

/**
 * Measurement interpolation to
 *
 * The currently defined unit values are:  "Linear", "Step", "Log"; Default: Linear
 */
UENUM(BlueprintType)
enum class EDMXGDTFMeasurementInterpolationTo : uint8
{
	Linear,
	Step,
	Log,

	// Max value. Add further elements above.
	MaxEnumValue UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EDMXGDTFMeasurementInterpolationTo, EDMXGDTFMeasurementInterpolationTo::MaxEnumValue);

namespace UE::DMX::GDTF
{
	class FDMXGDTFEmitter;
	class FDMXGDTFFilter;
	class FDMXGDTFMeasurementPoint;

	/** 
	* The measurement defines the relation between the requested output by a control channel and the physically achieved intensity. 
	* XML node for measurement is <Measurement>.
	* 
	* The order of the measurements corresponds to their ascending physical values.
	* Additional definition for additive color mixing: It is assumed that the physical value 0 exists and has zero output.
	* Additional definition for subtractive color mixing: The flag is removed with physical value 0 and it does not affect the beam. Physical value 100 is maximally inserted and affects the beam.
	*
	* Note 1: Some fixtures may vary in color response. These fixtures define multiple measurement points and corresponding interpolations.
	*/
	class DMXGDTF_API FDMXGDTFMeasurementBase
		: public FDMXGDTFNode
	{
	public:
		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Measurement"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface
		
		/** 
		 * For additive color mixing: uniquely given emitter intensity DMX percentage. Value range between > 0 and <= 100. 
		 * For subtractive color mixing: uniquely given flag insertion DMX percentage. Value range between 0 and 100. 
		 */
		float Physical = 0.f;		

		/** Used for additive color mixing: overall candela value for the enclosed set of measurement. */
		float LuminousIntensity = 0.f;		

		/** Used for subtractive color mixing: total amount of lighting energy passed at this insertion percentage. */
		float Transmission = 0.f;	

		/** Interpolation scheme from the previous value.The currently defined values are : "Linear", "Step", "Log"; Default: Linear */
		EDMXGDTFMeasurementInterpolationTo InterpolationTo = EDMXGDTFMeasurementInterpolationTo::Linear;

		/** As children the Measurement Collect has an optional list of a measurement point. */
		TArray<TSharedPtr<FDMXGDTFMeasurementPoint>> MeasurementPointArray;
	};

	/** Measurement for emitters */
	class DMXGDTF_API FDMXGDTFEmitterMeasurement
		: public FDMXGDTFMeasurementBase
	{
	public:
		FDMXGDTFEmitterMeasurement(const TSharedRef<FDMXGDTFEmitter>& InEmitter);

		/** The outer emitter */
		const TWeakPtr<FDMXGDTFEmitter> OuterEmitter;
	};

	/** Measurement for filters */
	class DMXGDTF_API FDMXGDTFFilterMeasurement
		: public FDMXGDTFMeasurementBase
	{
	public:
		FDMXGDTFFilterMeasurement(const TSharedRef<FDMXGDTFFilter>& InEmitter);

		/** The outer filter */
		const TWeakPtr<FDMXGDTFFilter> OuterFilter;
	};
}
