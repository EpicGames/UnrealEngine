// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFMeasurementBase;

	/** 
	 * The measurement point defines the energy of a specific wavelength of a spectrum. The XML node for measurement point is <MeasurementPoint>. 
	 * 
	 * It is recommended, but not required, that measurement points are evenly spaced. Regions with minimal light energy can be omitted, but the 
	 * decisive range of spectrum must be included. Recommended measurement spacing is 1 nm. Measurement spacing should not exceed 4 nm. 
	 */
	class DMXGDTF_API FDMXGDTFMeasurementPoint
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFMeasurementPoint(const TSharedRef<FDMXGDTFMeasurementBase>& InMeasurement);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("MeasurementPoint"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Center wavelength of measurement(nm). */
		float WaveLength;	

		/** Lighting energy(W / m2 / nm) */
		float Energy;

		/** The outer measurement */
		const TWeakPtr<FDMXGDTFMeasurementBase> OuterMeasurement;
	};
}
