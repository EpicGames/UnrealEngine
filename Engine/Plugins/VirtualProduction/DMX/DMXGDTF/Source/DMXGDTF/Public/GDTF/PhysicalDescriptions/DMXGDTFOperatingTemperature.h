// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFProperties;

	/** This section defines the description of the OperatingTemperature (XML node <OperatingTemperature>). */
	class DMXGDTF_API FDMXGDTFOperatingTemperature
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFOperatingTemperature(const TSharedRef<FDMXGDTFProperties>& InProperties);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("OperatingTemperature"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Lowest temperature the device can be operated.Unit: °C. Default value: 0 */
		float Low = 0.f;

		/** Highest temperature the device can be operated.Unit: °C. Default value: 40 */
		float High = 40.f;
	
		/** The outer properties */
		const TWeakPtr<FDMXGDTFProperties> OuterProperties;
	};
}
