// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFColorSpace;
	class FDMXGDTFColorRenderingIndexGroup;
	class FDMXGDTFConnector;
	class FDMXGDTFDMXProfile;
	class FDMXGDTFEmitter;
	class FDMXGDTFFilter;
	class FDMXGDTFFixtureType;
	class FDMXGDTFGamut;
	class FDMXGDTFProperties;

	/** This section describes the physical constitution of the device. It currently does not have any XML Attributes (XML node <PhysicalDescriptions>). */
	class DMXGDTF_API FDMXGDTFPhysicalDescriptions
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFPhysicalDescriptions(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("PhysicalDescriptions"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** 
		 * This section contains the description of the emitters. Emitter Collect defines additive 
		 * mixing of light sources, such as LEDs and tungsten lamps with permanently fitted filters. 
		 * It currently does not have any XML Attributes (XML node <Emitters>). 
		 */
		TArray<TSharedPtr<FDMXGDTFEmitter>> Emitters;

		/** 
		 * This section contains the description of the filters. The Filter Collect defines subtractive 
		 * mixing of light sources by filters, such as subtractive mixing flags and media used in physical 
		 * or virtual Color Wheels. It currently does not have any XML Attributes (XML node <Filters>).
		 */
		TArray<TSharedPtr<FDMXGDTFFilter>> Filters;

		/** This section defines color spaces. Currently it does not have any XML attributes (XML node <AdditionalColorSpaces>). */
		TSharedPtr<FDMXGDTFColorSpace> ColorSpaces;

		/** Describes additional device color spaces. */
		TArray<TSharedPtr<FDMXGDTFColorSpace>> AdditionalColorSpaces;

		/** This section defines gamuts. */
		TArray<TSharedPtr<FDMXGDTFGamut>> Gamuts;

		/** This section defines DMX profile descriptions. */
		TArray<TSharedPtr<FDMXGDTFDMXProfile>> DMXProfiles;

		/** This section contains TM-30-15 Fidelity Index (Rf) for 99 color samples. Currently it does not have any XML attributes (XML node <CRIs>). */
		TArray<TSharedPtr<FDMXGDTFColorRenderingIndexGroup>> CRIs;

		/** Describes physical properties of the device. */
		TSharedPtr<FDMXGDTFProperties> Properties;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;
	};
}
