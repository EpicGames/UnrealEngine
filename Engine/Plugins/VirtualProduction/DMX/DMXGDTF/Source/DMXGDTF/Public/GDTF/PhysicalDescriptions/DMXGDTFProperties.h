// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFLegHeight;
	class FDMXGDTFOperatingTemperature;
	class FDMXGDTFPhysicalDescriptions;
	class FDMXGDTFWeight;

	/** 
	 * This section defines the description of the Properties (XML node <Properties>). 
	 * 
	 * Connectors are obsolete as of GDTF 1.2 and are not implemented.
	 */
	class DMXGDTF_API FDMXGDTFProperties
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFProperties(const TSharedRef<FDMXGDTFPhysicalDescriptions>& InPhysicalDescriptions);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Properties"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** (optional) Temperature range in which the device can be operated. */
		TSharedPtr<FDMXGDTFOperatingTemperature> OperatingTemperature;

		/** (optional) Weight of the device including all accessories. */
		TSharedPtr<FDMXGDTFWeight> Weight;

		/** (optional) Height of the legs. */
		TSharedPtr<FDMXGDTFLegHeight> LegHeight;

		/** The outer physcial descriptions */
		const TWeakPtr<FDMXGDTFPhysicalDescriptions> OuterPhysicalDescriptions;
	};
}
