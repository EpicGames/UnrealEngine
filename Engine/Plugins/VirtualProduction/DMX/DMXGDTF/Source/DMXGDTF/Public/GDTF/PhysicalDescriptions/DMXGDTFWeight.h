// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFProperties;

	/** This section defines the overall weight of the device (XML node <Weight>). */
	class DMXGDTF_API FDMXGDTFWeight
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFWeight(const TSharedRef<FDMXGDTFProperties>& InProperties);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Weight"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Weight of the device including all accessories. Unit: kilogram. Default value: 0 */
		float Value = 0.f;	
	
		/** The outer properties */
		const TWeakPtr<FDMXGDTFProperties> OuterProperties;
	};
}
