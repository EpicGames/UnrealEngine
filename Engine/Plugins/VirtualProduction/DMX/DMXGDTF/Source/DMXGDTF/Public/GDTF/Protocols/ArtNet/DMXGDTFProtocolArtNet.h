// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFProtocols;

	namespace ArtNet
	{
		class FDMXGDTFProtocolArtNetDMXMap;

		/** If the device supports the Art-Net protocol, this section defines the corresponding information (XML node <Art-Net>). */
		class DMXGDTF_API FDMXGDTFProtocolArtNet
			: public FDMXGDTFNode
		{
		public:
			FDMXGDTFProtocolArtNet(const TSharedRef<FDMXGDTFProtocols>& InProtocols);

			//~ Begin FDMXGDTFNode interface
			virtual const TCHAR* GetXmlTag() const override { return TEXT("FTRDM"); }
			virtual void Initialize(const FXmlNode& XmlNode) override;
			virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
			//~ End FDMXGDTFNode interface

			/** As children the Art-Net has a list of Maps. */
			TArray<TSharedPtr<FDMXGDTFProtocolArtNetDMXMap>> Maps;

			/** The outer protocols */
			const TWeakPtr<FDMXGDTFProtocols> OuterProtocols;
		};
	}
}
