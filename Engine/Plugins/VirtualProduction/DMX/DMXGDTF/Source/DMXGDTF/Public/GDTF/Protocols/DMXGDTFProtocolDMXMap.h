// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	/**
	 * To define a custom mapping between Streaming ACN values and DMX Stream values you can add an XML node <Map> as a child.
	 * By default, it is assumed that all the values are mapped 1:1, so only when you differ from that you can add a custom map.
	 */
	class DMXGDTF_API FDMXGDTFProtocolDMXMapBase
		: public FDMXGDTFNode
	{
	public:
		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Protocols"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Value of the protocol value. */
		uint16 Key = 0;

		/** Value of the DMX value. */
		uint16 Value = 0;
	};

	namespace ArtNet
	{
		class FDMXGDTFProtocolArtNet;

		class DMXGDTF_API FDMXGDTFProtocolArtNetDMXMap
			: public FDMXGDTFProtocolDMXMapBase
		{
		public:
			FDMXGDTFProtocolArtNetDMXMap(const TSharedRef<ArtNet::FDMXGDTFProtocolArtNet>& InProtocolArtNet);

			const TWeakPtr<FDMXGDTFProtocolArtNet> OuterProtocolArtNet;
		};
	}

	namespace SACN
	{
		class FDMXGDTFProtocolSACN;

		class DMXGDTF_API FDMXGDTFProtocolSACNDMXMap
			: public FDMXGDTFProtocolDMXMapBase
		{
		public:
			FDMXGDTFProtocolSACNDMXMap(const TSharedRef<FDMXGDTFProtocolSACN>& InProtocolArtNet);

			const TWeakPtr<FDMXGDTFProtocolSACN> OuterProtocolSACN;
		};
	}
}