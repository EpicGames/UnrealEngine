// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;
	namespace RDM { class FDMXGDTFProtocolFTRDM; }
	namespace ArtNet { class FDMXGDTFProtocolArtNet; }
	namespace SACN { class FDMXGDTFProtocolSACN; }

	/** This section defines the overall Protocols of the device (XML node <Protocols>). */
	class DMXGDTF_API FDMXGDTFProtocols
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFProtocols(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Protocols"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Describes RDM information */
		TSharedPtr<RDM::FDMXGDTFProtocolFTRDM> RDM;

		/** Describes Art-Net information */
		TSharedPtr<ArtNet::FDMXGDTFProtocolArtNet> ArtNet;

		/** Describes sACN information */
		TSharedPtr<SACN::FDMXGDTFProtocolSACN> sACN;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;
	};
}
