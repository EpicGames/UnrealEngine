// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFDMXMode;

	namespace RDM
	{
		class FDMXGDTFSoftwareVersionID;

		/** For each supported software version add an XML node <DMXPersonality>.. */
		class DMXGDTF_API FDMXGDTFDMXPersonality
			: public FDMXGDTFNode
		{
		public:
			FDMXGDTFDMXPersonality(const TSharedRef<FDMXGDTFSoftwareVersionID>& InSoftwareVersionID);

			//~ Begin FDMXGDTFNode interface
			virtual const TCHAR* GetXmlTag() const override { return TEXT("DMXPersonality"); }
			virtual void Initialize(const FXmlNode& XmlNode) override;
			virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
			//~ End FDMXGDTFNode interface

			/** Hex Value of the DMXPersonality */
			uint32 Value = 0;

			/** Link to the DMX Mode that can be used with this software version. */
			FName DMXMode;

			/** Resolves the linked DMX mode. Returns the DMX mode, or nullptr if no DMX mode is linked. */
			TSharedPtr<FDMXGDTFDMXMode> ResolveDMXMode() const;

			/** The outer protocols */
			const TWeakPtr<FDMXGDTFSoftwareVersionID> OuterSoftwareVersionID;
		};
	}
}
