// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF { class FDMXGDTFProtocols; }

namespace UE::DMX::GDTF::RDM
{
	class FDMXGDTFSoftwareVersionID;

	/** This section defines the overall ProtocolRDM of the device (XML node <ProtocolRDM>). */
	class DMXGDTF_API FDMXGDTFProtocolFTRDM
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFProtocolFTRDM(const TSharedRef<FDMXGDTFProtocols>& InProtocols);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("FTRDM"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Manufacturer ESTA ID */
		uint32 ManufacturerID = 0;	

		/** Unique device model ID */
		uint32 DeviceModelID = 0;

		/** As children the FTRDM has a list of SoftwareVersionID. */
		TArray<TSharedPtr<FDMXGDTFSoftwareVersionID>> SoftwareVersionIDArray;

		/** The outer protocols */
		const TWeakPtr<FDMXGDTFProtocols> OuterProtocols;
	};
}
