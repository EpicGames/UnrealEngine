// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF::RDM
{
	class FDMXGDTFDMXPersonality;
	class FDMXGDTFProtocolFTRDM;

	/** For each supported software version add an XML node <SoftwareVersionID>.. */
	class DMXGDTF_API FDMXGDTFSoftwareVersionID
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFSoftwareVersionID(const TSharedRef<FDMXGDTFProtocolFTRDM>& InProtocolRDM);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("SoftwareVersionID"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** Software version ID */
		uint32 Value = 0;

		/** As children the SoftwareVersionID has a list of DMXPersonality. */
		TArray<TSharedPtr<FDMXGDTFDMXPersonality>> DMXPersonalityArray;

		/** The outer protocols */
		const TWeakPtr<FDMXGDTFProtocolFTRDM> OuterProtocolRDM;
	};
}
