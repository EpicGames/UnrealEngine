// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFProtocols;

	namespace SACN
	{
		class FDMXGDTFProtocolSACNDMXMap;

		/** If the device supports the Art-Net protocol, this section defines the corresponding information (XML node <Art-Net>). */
		class DMXGDTF_API FDMXGDTFProtocolSACN
			: public FDMXGDTFNode
		{
		public:
			FDMXGDTFProtocolSACN(const TSharedRef<FDMXGDTFProtocols>& InProtocols);

			//~ Begin FDMXGDTFNode interface
			virtual const TCHAR* GetXmlTag() const override { return TEXT("sACN"); }
			virtual void Initialize(const FXmlNode& XmlNode) override;
			virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
			//~ End FDMXGDTFNode interface

			/** As children the Art-Net has a list of Maps. */
			TArray<TSharedPtr<FDMXGDTFProtocolSACNDMXMap>> Maps;

			/** The outer protocols */
			const TWeakPtr<FDMXGDTFProtocols> OuterProtocols;
		};
	}
}
