// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Misc/DateTime.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFFixtureType;

	/** 
	 * This section defines one revision of a the device type (XML node Revision>). 
	 * Revisions are optional. Every time a GDTF file is uploaded to the database, 
	 * a revision with the actual time and UserID is created by the database. 
	 */
	class DMXGDTF_API FDMXGDTFRevision
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFRevision(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Revision"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface;

		/** User-defined text for this revision; Default value: empty */
		FString Text;

		/** Revision date and time */
		FDateTime Date;

		/** UserID of the user that has uploaded the GDTF file to the database; Default value: 0 */
		uint32 UserID = 0;

		/** Name of the software that modified this revision; Default value: empty */
		FString ModifiedBy;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;

	private:
		/** Parses a GDTF string as date time */
		FDateTime ParseDateTime(const FString& GDTFString) const;
	};
}
