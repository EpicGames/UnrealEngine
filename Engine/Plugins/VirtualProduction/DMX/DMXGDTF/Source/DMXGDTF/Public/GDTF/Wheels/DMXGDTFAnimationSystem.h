// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector2D.h" 

namespace UE::DMX::GDTF
{
	class FDMXGDTFWheelSlot;

	/** This section can only be defined for the animation system disk and it describes the animation system behavior (XML node <AnimationSystem>). */
	class FDMXGDTFAnimationSystem
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFAnimationSystem(const TSharedRef<FDMXGDTFWheelSlot>& InWheelSlot);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("AnimationSystem"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/**
		 * First Point of the Spline describing the path of animation system in the
		 * beam in relation to the middle of the Media File; Array of two floats;
		 * Seperator of values is “,”;First Float is X-axis and second is Y-axis.
		 * 
		 * UE specific: Using a FVector2D instead of an array of floats with two elements.
		 */
		FVector2D P1;

		/** 
		 * Second Point of the Spline describing the path of animation system in the 
 		 * beam in relation to the middle of the Media File; Array of two floats; 
		 * Seperator of values is “,”; First Float is X-axis and second is Y-axis.
		 * 
 		 * UE specific: Using a FVector2D instead of an array of floats with two elements.
		 */
		FVector2D P2;

		/** 
		 * Third Point of the Spline describing the path of animation system in the 
		 * beam in relation to the middle of the Media File; Array of two floats; 
		 * Seperator of values is “,”; First Float is X-axis and second is Y-axis. 
		 * 
 		 * UE specific: Using a FVector2D instead of an array of floats with two elements.
		 */
		FVector2D P3;

		/**
		 * Radius of the circle that defines the section of the animation system which 
		 * will be shown in the beam 
		 */
		float Radius = 0.0;

		/** The outer wheel slot */
		const TWeakPtr<FDMXGDTFWheelSlot> OuterWheelSlot;
	};
}
