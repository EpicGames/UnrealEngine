// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"
#include "Math/Transform.h" 

namespace UE::DMX::GDTF
{
	class FDMXGDTFWheelSlot;

	/** This section can only be defined for the prism wheel slot and has a description for the prism facet (XML node <Facet>). */
	class DMXGDTF_API FDMXGDTFPrismFacet
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFPrismFacet(const TSharedRef<FDMXGDTFWheelSlot>& InWheelSlot);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Facet"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/**
		 * Color of prism facet.
		 *
		 * UE specific: Using a Vector3d to store the xyY color.
		 */
		FDMXGDTFColorCIE1931xyY Color = { 0.3127f, 0.3290f, 100.0f };

		/** Specify the rotation, translation and scaling for the facet. */
		FTransform Rotation = FTransform::Identity;

		/** The outer wheel slot */
		const TWeakPtr<FDMXGDTFWheelSlot> OuterWheelSlot;
	};
}
