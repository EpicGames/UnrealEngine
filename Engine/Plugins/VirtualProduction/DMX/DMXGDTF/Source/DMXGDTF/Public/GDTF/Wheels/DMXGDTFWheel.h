// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDTF/DMXGDTFNode.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFWheelSlot;

	/**
	 * Each wheel describes a single physical or virtual wheel of the fixture type.
	 * If the real device has wheels you can change, then all wheel configurations have to be described.
	 * Wheel has the following XML node: <Wheel>.
	 */
	class DMXGDTF_API FDMXGDTFWheel
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFWheel(const TSharedRef<FDMXGDTFFixtureType>& InFixtureType);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Wheel"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the wheel */
		FName Name;

		/** As children, Wheel has a list of wheel slots. */
		TArray<TSharedPtr<FDMXGDTFWheelSlot>> WheelSlotArray;

		/** The outer fixture type */
		const TWeakPtr<FDMXGDTFFixtureType> OuterFixtureType;
	};
}
