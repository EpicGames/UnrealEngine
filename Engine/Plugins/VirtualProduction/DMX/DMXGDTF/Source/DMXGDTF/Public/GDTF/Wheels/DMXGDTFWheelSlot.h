// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXGDTFColorCIE1931xyY.h"
#include "GDTF/DMXGDTFNode.h"
#include "Math/Vector.h"

namespace UE::DMX::GDTF
{
	class FDMXGDTFAnimationSystem;
	class FDMXGDTFPrismFacet;
	class FDMXGDTFWheel;

	/** The wheel slot represents a slot on the wheel (XML node <Slot>). */
	class DMXGDTF_API FDMXGDTFWheelSlot
		: public FDMXGDTFNode
	{
	public:
		FDMXGDTFWheelSlot(const TSharedRef<FDMXGDTFWheel>& InWheel);

		//~ Begin FDMXGDTFNode interface
		virtual const TCHAR* GetXmlTag() const override { return TEXT("Slot"); }
		virtual void Initialize(const FXmlNode& XmlNode) override;
		virtual FXmlNode* CreateXmlNode(FXmlNode& Parent) override;
		//~ End FDMXGDTFNode interface

		/** The unique name of the wheel slot */
		FName Name;

		/**
		 * Color of the wheel slot, Default value: {0.3127, 0.3290, 100.0} (white). 
		 * For Y give relative value compared to overall output defined in property
		 * Luminous Flux of related Beam Geometry (transmissive case).
		 */
		FDMXGDTFColorCIE1931xyY Color;

		/**
		 * (Optional) PNG file name without extension containing image for specific gobos etc.
		 *	— Maximum resolution of picture: 1024 × 1024;
		 *	— Recommended resolution of gobo: 256 × 256;
		 *	— Recommended resolution of animation wheel: 256 × 256
		 *	These resource files are located in a folder called ./wheels in the zip
		 *	archive. Default value: empty.
		 */
		FString MediaFileName;

		/** If the wheel slot has a prism, it has to have one or several children called prism facet. */
		TArray<TSharedPtr<FDMXGDTFPrismFacet>> PrismFacetArray;

		/** If the wheel slot has an AnimationWheel, it has to have one child called Animation Wheel. */
		TSharedPtr<FDMXGDTFAnimationSystem> AnimationWheel;

		/** The outer wheel */
		const TWeakPtr<FDMXGDTFWheel> OuterWheel;
	};
}
