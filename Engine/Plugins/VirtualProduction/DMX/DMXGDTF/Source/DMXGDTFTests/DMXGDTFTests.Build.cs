// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXGDTFTests : ModuleRules
{
	public DMXGDTFTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;			
	
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"DMXGDTF",
				"DMXZip",
				"Projects",
				"Slate",
				"SlateCore",
				"XmlParser"
			}
		);
	}
}
