// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXGDTFTests.h"

#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Containers/UnrealString.h"
#include "DesktopPlatformModule.h"
#include "DMXGDTF.h"
#include "DMXGDTFTests.h"
#include "DMXZipper.h"
#include "HAL/FileManagerGeneric.h"
#include "IDesktopPlatform.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "XmlFile.h"

DEFINE_LOG_CATEGORY(LogDMXGDTFTests);

#if WITH_DEV_AUTOMATION_TESTS 

namespace UE::DMX::GDTF
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDMXGDTFAutomationTest, "DMX.GDTF.ImportExport", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

		struct FDMXGDTFTestFile
	{
		FDMXGDTFTestFile(const FString& InFilename, const TArray64<uint8>& InData, const TSharedPtr<FXmlFile>& InXmlFile)
			: Filename(InFilename)
			, Data(InData)
			, XmlFile(InXmlFile)
		{}

		FString Filename;
		TArray64<uint8> Data;
		TSharedPtr<FXmlFile> XmlFile;
	};

	TArray<FDMXGDTFTestFile> LoadGDTFFiles(const FString& GDTFContentDir)
	{
		TArray<FString> GDTFFilenames;
		constexpr bool bFiles = true;
		constexpr bool bDirectories = false;
		IFileManager::Get().FindFiles(GDTFFilenames, *(GDTFContentDir / TEXT("*")), bFiles, bDirectories);
		
		GDTFFilenames.RemoveAll([](const FString& Filename)
			{
				return !Filename.EndsWith(TEXT(".gdtf"));
			});

		if (GDTFFilenames.IsEmpty())
		{
			UE_LOGF(LogDMXGDTFTests, Warning, "No GDTF files are present in DMXGDTFTests plugin content. Cannot run tests without files.");
			UE_LOGF(LogDMXGDTFTests, Warning, "Put one or more GDTFs into the content folder of the DMXGDTF plugin to test specific GDTFs.");
		}

		TArray<FDMXGDTFTestFile> Files;
		for (const FString& GDTFFilename : GDTFFilenames)
		{
			TArray64<uint8> Buffer;

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			IFileHandle* FileHandle = PlatformFile.OpenRead(*(GDTFContentDir / GDTFFilename));
			if (ensureMsgf(FileHandle, TEXT("Cannot open file %s."), *(GDTFContentDir / GDTFFilename)))
			{
				int64 FileSize = FileHandle->Size();
				Buffer.SetNumUninitialized(FileSize, EAllowShrinking::No);

				if (!FileHandle->Read(Buffer.GetData(), FileSize))
				{
					UE_LOGF(LogDMXGDTFTests, Warning, "Cannot open gdtf file '%ls', skipping file.", *GDTFFilename);
					continue;
				}

				FDMXZipper Zip;
				if (!Zip.LoadFromData(Buffer))
				{
					UE_LOGF(LogDMXGDTFTests, Warning, "Cannot unzip gdtf '%ls', skipping file.", *GDTFFilename);
					continue;
				}

				TArray64<uint8> DescriptionXmlData;
				if (!Zip.GetFileContent(TEXT("description.xml"), DescriptionXmlData))
				{
					UE_LOGF(LogDMXGDTFTests, Warning, "Cannot find description.xml in gdtf '%ls', skipping file.", *GDTFFilename);
					continue;
				}

				FString DescriptionXmlString;
				FFileHelper::BufferToString(DescriptionXmlString, DescriptionXmlData.GetData(), DescriptionXmlData.Num());

				const TSharedPtr<FXmlFile> XmlFile = MakeShared<FXmlFile>();
				if (!XmlFile->LoadFile(DescriptionXmlString, EConstructMethod::ConstructFromBuffer))
				{
					UE_LOGF(LogDMXGDTFTests, Warning, "Cannot read description.xml in '%ls', skipping file.", *GDTFFilename);
					continue;
				}

				Files.Add(FDMXGDTFTestFile(GDTFFilename, Buffer, XmlFile));
			}
		}

		return Files;
	}

	void GenerateTarget(TArray<FDMXGDTFTestFile>& InOutFiles)
	{
		UE_LOGF(LogDMXGDTFTests, Log, "Parsing GDTF files:");
		for (FDMXGDTFTestFile& File : InOutFiles)
		{
			UE_LOGF(LogDMXGDTFTests, Log, "		* %ls", *File.Filename);

			UDMXGDTF* GDTF = NewObject<UDMXGDTF>();
			GDTF->InitializeFromData(File.Data);

			File.XmlFile = GDTF->ExportAsXml();
		}
	}

	/** Compares any scalars, vectors and matrices. Returns false if the string is not a math object. */
	bool LexicallyCompareMathObjects(const FString& InSourceString, const FString& InTargetString, double Tolerance, bool& bOutEquals)
	{
		bOutEquals = true;
		if (InSourceString.IsEmpty() || InTargetString.IsEmpty())
		{
			return false;
		}

		// Handle matrices like vectors
		FString CleanSourceString;
		TArray<FString> MatrixSourceStringArray;
		InSourceString.ParseIntoArray(MatrixSourceStringArray, TEXT("}"));
		for (const FString& MatrixSourceString : MatrixSourceStringArray)
		{
			CleanSourceString += MatrixSourceString.Replace(TEXT("{"), TEXT(""));
		}

		FString CleanTargetString;
		TArray<FString> MatrixTargetStringArray;
		InTargetString.ParseIntoArray(MatrixTargetStringArray, TEXT("}"));
		for (const FString& MatrixTargetString : MatrixTargetStringArray)
		{
			CleanTargetString += MatrixTargetString.Replace(TEXT("{"), TEXT(""));
		}
		
		TArray<FString> SourceStringArray;
		CleanSourceString.ParseIntoArray(SourceStringArray, TEXT(","));

		TArray<FString> TargetStringArray;
		CleanTargetString.ParseIntoArray(TargetStringArray, TEXT(","));
		if (SourceStringArray.Num() != TargetStringArray.Num())
		{
			UE_LOGF(LogDMXGDTFTests, Warning, "Number of components in possible vector do not match, source is '%ls' but target is '%ls'", *InSourceString, *InTargetString);
			bOutEquals = false;
			return false;
		}

		for (int32 ComponentIndex = 0; ComponentIndex < SourceStringArray.Num(); ComponentIndex++)
		{
			double SourceComponent;
			double TargetComponent;
			if (LexTryParseString(SourceComponent, *SourceStringArray[ComponentIndex]) &&
				LexTryParseString(TargetComponent, *TargetStringArray[ComponentIndex]))
			{
				if (!FMath::IsNearlyEqual(SourceComponent, TargetComponent, Tolerance))
				{
					bOutEquals = false;
					break;
				}
			}
			else
			{
				// Not a math object
				return false;
			}
		}

		return true;
	}

	/** Returns true if a specific collect can be omitted when empty */
	bool CanOmitCollect(const FString& ParetNodeTag, const FString& CollectName)
	{
		if (ParetNodeTag == TEXT("PhysicalDescriptions"))
		{
			return
				CollectName == TEXT("AdditionalColorSpaces") ||
				CollectName == TEXT("Gamuts") ||
				CollectName == TEXT("FTMacros");
		}

		return false;
	}

	bool CanOmitDerprecatedAttribute(const FXmlNode* Node, const FXmlAttribute& Attribute)
	{
		check(Node);

		if (Node->GetTag() == TEXT("DMXChannel") &&
			Attribute.GetTag() == TEXT("Default"))
		{
			return true;
		}

		return false;
	}

	/** Some attributes can be omitted if they're set to their default value. Returns true for specific attributes that pass either test. */
	bool CanOmitAttributeWithDefaultValue(const FXmlNode* Node, const FXmlAttribute& Attribute)
	{
		check(Node);

		const auto AcceptOmittedDefault([Node, Attribute](const TCHAR* Tag, const TCHAR* AttributeName, const FString& DefaultValue)
			{
				if (Node->GetTag() == Tag)
				{
					const FXmlAttribute* AttributePtr = Algo::FindByPredicate(Node->GetAttributes(), [AttributeName](const FXmlAttribute& Attribute)
						{
							return Attribute.GetTag() == AttributeName;
						});

					// Both omitting or setting the default value is ok.
					return !AttributePtr || AttributePtr->GetValue() == DefaultValue;
				}
				return false;
			});

		const auto AcceptOmittedDefaults([Node, Attribute](const TCHAR* Tag, const TCHAR* AttributeName, const TArray<FString>& DefaultValues)
			{
				for (const FString& DefaultValue : DefaultValues)
				{
					if (Attribute.GetValue() == DefaultValue)
					{
						return true;
					}
				}
				return false;
			});

		// CanHaveChildren can be omitted in FixtureType when defaulted
		if (AcceptOmittedDefault(TEXT("FixtureType"), TEXT("CanHaveChildren"), TEXT("Yes")))
		{
			return true;
		}

		// Name can be omitted in ColorSpace when defaulted
		if (AcceptOmittedDefaults(TEXT("ColorSpace"), TEXT("Name"), { TEXT("Default"), TEXT("") }))
		{
			return true;
		}

		// Description can be omitted in DMXMode when defaulted
		if (AcceptOmittedDefault(TEXT("DMXMode"), TEXT("Description"), TEXT("")))
		{
			return true;
		}

		// ModeFrom and ModeTo can be omitted in ChannelFunction when defaulted
		if (AcceptOmittedDefault(TEXT("ChannelFunction"), TEXT("ModeFrom"), TEXT("0/1")) ||
			AcceptOmittedDefault(TEXT("ChannelFunction"), TEXT("ModeTo"), TEXT("0/1")) ||
			AcceptOmittedDefault(TEXT("ChannelFunction"), TEXT("CustomName"), TEXT("")))
		{
			return true;
		}

		// ModeFrom and ModeTo can be omitted in ChannelSet when defaulted
		if (AcceptOmittedDefault(TEXT("ChannelSet"), TEXT("ModeFrom"), TEXT("0/1")) ||
			AcceptOmittedDefault(TEXT("ChannelSet"), TEXT("ModeTo"), TEXT("0/1")))
		{
			return true;
		}

		// ModeFrom and ModeTo can be omitted in ChannelSet when defaulted
		if (AcceptOmittedDefaults(TEXT("Model"), TEXT("SVGOffsetX"), { TEXT("0"), TEXT("0.000000") }) ||
			AcceptOmittedDefaults(TEXT("Model"), TEXT("SVGOffsetY"), { TEXT("0"), TEXT("0.000000") }) ||
			AcceptOmittedDefaults(TEXT("Model"), TEXT("SVGSideOffsetX"), { TEXT("0"), TEXT("0.000000") }) ||
			AcceptOmittedDefaults(TEXT("Model"), TEXT("SVGSideOffsetY"), { TEXT("0"), TEXT("0.000000") }) ||
			AcceptOmittedDefaults(TEXT("Model"), TEXT("SVGFrontOffsetX"), { TEXT("0"), TEXT("0.000000") }) ||
			AcceptOmittedDefaults(TEXT("Model"), TEXT("SVGFrontOffsetY"), { TEXT("0"), TEXT("0.000000") }))
		{
			return true;
		}

		// ThrowRatio can be omitted in Beam when defaulted
		if (AcceptOmittedDefaults(TEXT("Beam"), TEXT("ThrowRatio"), { TEXT("1"), TEXT("1.000000") }) ||
			AcceptOmittedDefaults(TEXT("Beam"), TEXT("RectangleRatio"), { TEXT("1.7777"), TEXT("1.777700") }))
		{
			return true;
		}

		// ModifiedBy can be omitted in Revision when defaulted
		if (AcceptOmittedDefault(TEXT("Revision"), TEXT("ModifiedBy"), TEXT("")))
		{
			return true;
		}

		return false;
	}

	void LogChildCountMismatch(const FString& SourceNodeTag, const FString& TargetNodeTag, const TArray<FXmlNode*>& SourceArray, const TArray<FXmlNode*>& TargetArray)
	{
		if (!ensureMsgf(SourceNodeTag == TargetNodeTag, TEXT("Trying to log a mismatch in child count, but target nodes differ, source is '%s' and target '%s'"), *SourceNodeTag, *TargetNodeTag))
		{
			return;
		}

		TArray<const FXmlNode*> NodesOnlyPresentInSource;
		for (const FXmlNode* SourceChild : SourceArray)
		{
			if (!CanOmitCollect(SourceNodeTag, SourceChild->GetTag()) &&
				Algo::FindBy(TargetArray, SourceChild->GetTag(), &FXmlNode::GetTag) == nullptr)
			{
				NodesOnlyPresentInSource.Add(SourceChild);
			}
		}

		TArray<const FXmlNode*> NodesOnlyPresentInTarget;
		for (const FXmlNode* TargetChild : TargetArray)
		{
			if (!CanOmitCollect(TargetNodeTag, TargetChild->GetTag()) &&
				Algo::FindBy(SourceArray, TargetChild->GetTag(), &FXmlNode::GetTag) == nullptr)
			{
				NodesOnlyPresentInTarget.Add(TargetChild);
			}
		}

		if (!NodesOnlyPresentInSource.IsEmpty() || !NodesOnlyPresentInTarget.IsEmpty())
		{
			UE_LOGF(LogDMXGDTFTests, Warning, "Detected child count mismatch with source node '%ls' and target node '%ls':", *SourceNodeTag, *TargetNodeTag);

			for (const FXmlNode* PresentInSource : NodesOnlyPresentInSource)
			{
				UE_LOGF(LogDMXGDTFTests, Warning, "XML node only present in source:     * %ls", *PresentInSource->GetTag());
			}

			for (const FXmlNode* PresentInTarget : NodesOnlyPresentInTarget)
			{
				UE_LOGF(LogDMXGDTFTests, Warning, "XML node only present in target:     * %ls", *PresentInTarget->GetTag());
			}
		}
	}

	bool DiffEachChild(const FXmlNode* SourceStartNode, const FXmlNode* TargetStartNode, TFunctionRef<bool(const FXmlNode*, const FXmlNode*)> DiffPredicate)
	{
		check(SourceStartNode && TargetStartNode);

		struct FChildGroup
		{
			TArray<FXmlNode*> SourceArray;
			TArray<FXmlNode*> TargetArray;
		};

		TMap<FString, FChildGroup> TagToChildGroupMap;
		for (FXmlNode* SourceChild : SourceStartNode->GetChildrenNodes())
		{
			TagToChildGroupMap.FindOrAdd(SourceChild->GetTag()).SourceArray.Add(SourceChild);
		}
		for (FXmlNode* TargetChild : TargetStartNode->GetChildrenNodes())
		{
			TagToChildGroupMap.FindOrAdd(TargetChild->GetTag()).TargetArray.Add(TargetChild);
		}

		// Ignore deprecated nodes that are not implemented in unreal engine
		static const TArray<FString> NodesWithoutImplementation = { TEXT("Connectors") };
		for (const FString& NodeWithoutImplementation : NodesWithoutImplementation)
		{
			TagToChildGroupMap.Remove(NodeWithoutImplementation);
		}

		bool bSuccess = true;
		for (const TTuple<FString, FChildGroup>& TagToChildGroupPair : TagToChildGroupMap)
		{
			// Test child count mismatch
			if (TagToChildGroupPair.Value.SourceArray.Num() != TagToChildGroupPair.Value.TargetArray.Num())
			{
				LogChildCountMismatch(SourceStartNode->GetTag(), TargetStartNode->GetTag(), TagToChildGroupPair.Value.SourceArray, TagToChildGroupPair.Value.TargetArray);
				bSuccess = false;

				continue;
			}

			// Diff children recursive
			for (int32 ChildIndex = 0; ChildIndex < TagToChildGroupPair.Value.SourceArray.Num(); ChildIndex++)
			{
				check(TagToChildGroupPair.Value.SourceArray.IsValidIndex(ChildIndex) && TagToChildGroupPair.Value.TargetArray.IsValidIndex(ChildIndex));

				FXmlNode* SourceChild = TagToChildGroupPair.Value.SourceArray[ChildIndex];
				FXmlNode* TargetChild = TagToChildGroupPair.Value.TargetArray[ChildIndex];

				bSuccess &= DiffPredicate(SourceChild, TargetChild);

				bSuccess &= DiffEachChild(SourceChild, TargetChild, DiffPredicate);
			}
		}

		return bSuccess;
	}

	bool DiffGDTFs(FDMXGDTFTestFile& Source, FDMXGDTFTestFile& Target, const FString& ExportDir)
	{
		check(Source.Filename == Target.Filename && Source.XmlFile.IsValid() && Target.XmlFile.IsValid());

		const FString Filename = Source.Filename;
		UE_LOGF(LogDMXGDTFTests, Log, "Testing GDTF file %ls", *Filename);

		const FXmlNode* SourceRoot = Source.XmlFile->GetRootNode();
		const FXmlNode* TargetRoot = Target.XmlFile->GetRootNode();

		// Diff root nodes
		if (SourceRoot->GetChildrenNodes().Num() != TargetRoot->GetChildrenNodes().Num())
		{
			LogChildCountMismatch(TEXT("RootNode"), TEXT("RootNode"), SourceRoot->GetChildrenNodes(), TargetRoot->GetChildrenNodes());
		}

		// Diff children recursive
		const bool bSuccess = DiffEachChild(SourceRoot, TargetRoot, [](const FXmlNode* Source, const FXmlNode* Target)
			{
				check(Source && Target);

				// Same tags
				const FString SourceTag = Source->GetTag();
				const FString TargetTag = Target->GetTag();

				// Same attributes
				for (const FXmlAttribute& SourceAttribute : Source->GetAttributes())
				{
					const FXmlAttribute* TargetAttributePtr = Algo::FindByPredicate(Target->GetAttributes(), [SourceAttribute](const FXmlAttribute& TargetAttribute)
						{
							return SourceAttribute.GetTag() == TargetAttribute.GetTag();
						});

					if (!TargetAttributePtr)
					{
						if (CanOmitDerprecatedAttribute(Source, SourceAttribute) ||
							CanOmitAttributeWithDefaultValue(Source, SourceAttribute))
						{
							return true;
						}
						else
						{
							UE_LOGF(LogDMXGDTFTests, Warning, "Cannot find attribute '%ls' in target with node '%ls'.", *SourceAttribute.GetTag(), *SourceTag);
							return false;
						}
					}

					// Equal strings
					if ((*TargetAttributePtr).GetValue() != SourceAttribute.GetValue())
					{						
						// Equal numerical values and n-dimensional vectors with tolerance 
						bool bVectorsEqual = true;
						bool bMatricesEqual = true;
						constexpr double NumericalTolerance = 0.01;
						if (LexicallyCompareMathObjects(*SourceAttribute.GetValue(), *(*TargetAttributePtr).GetValue(), NumericalTolerance, bVectorsEqual))
						{
							if (!bVectorsEqual)
							{
								UE_LOGF(LogDMXGDTFTests, Warning, "Source value is '%ls' but target value is '%ls' for attribute '%ls' in node '%ls'.", *SourceAttribute.GetValue(), *(*TargetAttributePtr).GetValue(), *SourceAttribute.GetTag(), *SourceTag);
								return false;
							}
						}
						else if(!CanOmitDerprecatedAttribute(Source, SourceAttribute) ||
							!CanOmitAttributeWithDefaultValue(Target, *TargetAttributePtr))
						{
							UE_LOGF(LogDMXGDTFTests, Warning, "Source value is '%ls' but target value is '%ls' for attribute '%ls' in node '%ls'.", *SourceAttribute.GetValue(), *(*TargetAttributePtr).GetValue(), *SourceAttribute.GetTag(), *SourceTag);
							return false;
						}
					}
				}

				for (const FXmlAttribute& TargetAttribute : Target->GetAttributes())
				{
					const FXmlAttribute* SourceAttributePtr = Algo::FindByPredicate(Source->GetAttributes(), [TargetAttribute](const FXmlAttribute& SourceAttribute)
						{
							return TargetAttribute.GetTag() == SourceAttribute.GetTag();
						});

					if (!SourceAttributePtr && !CanOmitAttributeWithDefaultValue(Target, TargetAttribute))
					{
						UE_LOGF(LogDMXGDTFTests, Warning, "Cannot find attribute '%ls' in source with node '%ls'. Maybe node initializer sets the wrong default.", *TargetAttribute.GetTag(), *TargetTag);
						return false;
					}
					// Value mismatch is already tested above
				}

				return true;
			});

		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("DMXGDTF"));
		if (!ensureMsgf(Plugin, TEXT("Unexpected DMXGDTF plugin is not valid")))
		{
			return false;
		}

		const FString PluginContentDir = Plugin->GetContentDir();
		if (!ensureMsgf(FPaths::DirectoryExists(PluginContentDir), TEXT("Cannot find content dir for test.")))
		{
			return false;
		}

		Source.XmlFile->Save(ExportDir + Source.Filename.RightPad(4) + TEXT("_Source.xml"));
		Target.XmlFile->Save(ExportDir + Target.Filename.RightPad(4) + TEXT("_Target.xml"));

		return bSuccess;
	}

	bool FDMXGDTFAutomationTest::RunTest(const FString& Parameters)
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("DMXGDTF"));
		if (!Plugin.IsValid())
		{
			return false;
		}

		const FString PluginContentDir = Plugin->GetContentDir();
		if (!TestTrue("Cannot find content dir for test.", FPaths::DirectoryExists(PluginContentDir)))
		{
			return false;
		}

		const TArray<FDMXGDTFTestFile> Files = LoadGDTFFiles(PluginContentDir);
		if (Files.IsEmpty())
		{
			AddError(FString::Printf(TEXT("Cannot load GDTF files to run DMX GDTF Tests. See log for details.")));

			return false;
		}

		const FString ExportDir = PluginContentDir / TEXT("Results/");
		if (IFileManager::Get().DirectoryExists(*ExportDir))
		{
			constexpr bool bTree = true;
			IFileManager::Get().MakeDirectory(*ExportDir, bTree);
		}

		TArray<FDMXGDTFTestFile> Source = Files;
		TArray<FDMXGDTFTestFile> Target = Files;

		GenerateTarget(Target);
		
		int32 NumSuccessfulTests = 0;
		for (int32 FileIndex = 0; FileIndex < Source.Num(); FileIndex++)
		{
			AddInfo(FString::Printf(TEXT("*** DMX GDTF Tests: Testing gdtf '%s' ***"), *Source[FileIndex].Filename));
			if (DiffGDTFs(Source[FileIndex], Target[FileIndex], ExportDir))
			{
				NumSuccessfulTests++;
			}
		}

		constexpr bool bCaptureStack = true;
		AddInfo(TEXT("************************************************************"));
		AddInfo(FString::Printf(TEXT("*** DMX GDTF Tests: Tested %i/%i gdtf files successfully ***"), NumSuccessfulTests, Source.Num()), bCaptureStack);
		AddInfo(FString::Printf(TEXT("*** DMX GDTF Tests: Test results are export to '%s'."), *ExportDir), bCaptureStack);
		AddInfo(TEXT("************************************************************"));

		return NumSuccessfulTests == Source.Num();
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
