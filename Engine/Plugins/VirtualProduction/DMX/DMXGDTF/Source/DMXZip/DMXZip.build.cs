// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXZip : ModuleRules
{
	public DMXZip(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{}
		);
	}
}
