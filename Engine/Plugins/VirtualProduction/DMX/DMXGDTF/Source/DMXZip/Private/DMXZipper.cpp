// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXZipper.h"

#include "DMXZipLog.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"


FDMXZipper::FDMXZipper()
{
	SizeOfCentralDirectory = 0;
	OffsetOfCentralDirectory = 0;
	TotalNumberOfCentralDirectoryRecords = 0;
}

bool FDMXZipper::LoadFromFile(const FString& Filename)
{
	// Reset the reader
	Reader.Empty();

	if (!FFileHelper::LoadFileToArray(Reader, *Filename))
	{
		return false;
	}

	return Parse();
}

bool FDMXZipper::LoadFromData(const TArray64<uint8>& Data)
{
	// Reset the reader
	Reader.Empty();

	Reader.Append(Data);

	return Parse();
}

bool FDMXZipper::SaveToFile(const FString& Filename)
{
	TArray64<uint8> Data;
	if (GetData(Data))
	{
		return FFileHelper::SaveArrayToFile(Data, *Filename);
	}
	
	return false;
}

bool FDMXZipper::GetData(TArray64<uint8>& OutData)
{
	FArrayWriter Writer;

	OffsetOfCentralDirectory = 0;
	SizeOfCentralDirectory = 0;
	TotalNumberOfCentralDirectoryRecords = 0;

	for (const TPair<FString, TPair<TArray64<uint8>, bool>>& Pair : NewFilesToShouldCompressMap)
	{
		const bool bFileAdded = AddFileInternal(Writer, Pair.Key, Pair.Value.Key, Pair.Value.Value);
		if (!bFileAdded)
		{
			UE_LOGF(LogDMXZip, Error, "Cannot save Zip. File '%ls' added no longer exists on disk!", *Pair.Key);
			return false;
		}
	}

	for (const TPair<FString, uint32>& Pair : OffsetsMap)
	{
		// skip overwritten files
		if (NewFilesToShouldCompressMap.Contains(Pair.Key))
		{
			continue;
		}
		TArray64<uint8> Data;
		if (!GetFileContent(Pair.Key, Data))
		{
			UE_LOGF(LogDMXZip, Error, "Cannot save Zip.Failed to read File '%ls' from Zip when saving it.", *Pair.Key);
			return false;
		}

		if (!AddFileInternal(Writer, Pair.Key, Data, true))
		{
			return false;
		}
	}

	AddEndOfCentralDirectory(Writer);

	OutData = Writer;
	return true;
}

TArray<FString> FDMXZipper::GetFiles() const
{
	TArray<FString> Files;
	OffsetsMap.GetKeys(Files);
	return Files;
}

void FDMXZipper::AddFile(const FString& RelativeFilePathAndName, const TArray64<uint8>& Data, const bool bCompress)
{
	NewFilesToShouldCompressMap.Add(RelativeFilePathAndName, { Data, bCompress });
}

bool FDMXZipper::GetFileContent(const FString& Filename, TArray64<uint8>& OutData)
{
	uint32* Offset = OffsetsMap.Find(Filename);
	if (!Offset)
	{
		return false;
	}

	constexpr uint64 LocalEntryMinSize = 30;

	if (*Offset + LocalEntryMinSize > Reader.Num())
	{
		return false;
	}

	uint16 Compression = 0;
	uint32 CompressedSize;
	uint32 UncompressedSize = 0;
	uint16 FilenameLen = 0;
	uint16 ExtraFieldLen = 0;

	// Seek to Compression
	Reader.Seek(*Offset + 8);
	Reader << Compression;
	// Seek to CompressedSize
	Reader.Seek(*Offset + 18);
	Reader << CompressedSize;
	Reader << UncompressedSize;
	Reader << FilenameLen;
	Reader << ExtraFieldLen;

	if (*Offset + LocalEntryMinSize + FilenameLen + ExtraFieldLen + CompressedSize > Reader.Num())
	{
		return false;
	}

	if (Compression == 8)
	{
		OutData.AddUninitialized(UncompressedSize);
		if (!FCompression::UncompressMemory(NAME_Zlib, OutData.GetData(), UncompressedSize, Reader.GetData() + *Offset + LocalEntryMinSize + FilenameLen + ExtraFieldLen, CompressedSize, COMPRESS_NoFlags, -15))
		{
			return false;
		}
	}
	else if (Compression == 0 && CompressedSize == UncompressedSize)
	{
		OutData.Append(Reader.GetData() + *Offset + LocalEntryMinSize + FilenameLen + ExtraFieldLen, UncompressedSize);
	}
	else
	{
		return false;
	}

	return true;
}

FDMXZipper::FDMXScopedUnzipToTempFile::FDMXScopedUnzipToTempFile(const TSharedRef<FDMXZipper>& DMXZipper, const FString& FilenameInZip)
{
	if (!ensureAlwaysMsgf(DMXZipper->GetFiles().Contains(FilenameInZip), TEXT("Tried to unzip '%s', but zip does not contain the file."), *FilenameInZip))
	{
		return;
	}
	TArray64<uint8> FileData;
	if (!ensureAlwaysMsgf(DMXZipper->GetFileContent(FilenameInZip, FileData), TEXT("Tried to unzip '%s', but zip does not contain the file."), *FilenameInZip))
	{
		return;
	}

	const FString Directory = FPaths::EngineIntermediateDir() / TEXT("DMX_Temp");
	TempFilePathAndName = FPaths::ConvertRelativePathToFull(Directory / FilenameInZip);

	FFileHelper::SaveArrayToFile(FileData, *TempFilePathAndName);
}

FDMXZipper::FDMXScopedUnzipToTempFile::~FDMXScopedUnzipToTempFile()
{
	// Delete the extracted file
	IFileManager& FileManager = IFileManager::Get();
	constexpr bool bRequireExists = true;
	constexpr bool bEvenIfReadOnly = false;
	constexpr bool bQuiet = true;
	ensureAlwaysMsgf(FileManager.Delete(*TempFilePathAndName, bRequireExists, bEvenIfReadOnly, bQuiet), TEXT("Unexpected failed to delete temp file when using FDMXScopedUnzipToTempFile"));
}

bool FDMXZipper::AddFileInternal(FArrayWriter& Writer, const FString& Path, const TArray64<uint8>& Data, const bool bCompress)
{
	constexpr uint16 DeflateMode = 8;

	uint32 LocalFileHeaderSignature = 0x04034b50;
	uint16 VersionNeededToExtract = 0;
	uint16 GeneralPurposeFlags = 0;
	uint16 CompressionMethod = bCompress ? DeflateMode : 0;
	uint16 FileLastModificationTime = 0;
	uint16 FileLastModificationDate = 0;
	uint32 Crc = FCrc::MemCrc32(Data.GetData(), Data.Num(), 0);
	uint32 CompressedSize = Data.Num();
	uint32 UncompressedSize = Data.Num();
	uint16 FilenameLength = FCStringUtf8::Strlen(reinterpret_cast<UTF8CHAR*>(TCHAR_TO_UTF8(*Path)));
	uint16 ExtraFieldLength = 0;

	TArray64<uint8> CompressedData;

	if (bCompress)
	{
		int32 CompressionBounds = FCompression::CompressMemoryBound(NAME_Zlib, Data.Num());
		CompressedData.AddUninitialized(CompressionBounds);
		if (!FCompression::CompressMemory(NAME_Zlib, CompressedData.GetData(), CompressionBounds, Data.GetData(), Data.Num(), COMPRESS_NoFlags, -15))
		{
			return false;
		}
		CompressedSize = CompressionBounds;
	}

	uint32 LocalFileEntrySize = 30 + FilenameLength + (bCompress ? CompressedSize : Data.Num());

	Writer.InsertZeroed(OffsetOfCentralDirectory, LocalFileEntrySize);
	Writer.Seek(OffsetOfCentralDirectory);

	Writer << LocalFileHeaderSignature;
	Writer << VersionNeededToExtract;
	Writer << GeneralPurposeFlags;
	Writer << CompressionMethod;
	Writer << FileLastModificationTime;
	Writer << FileLastModificationDate;
	Writer << Crc;
	Writer << CompressedSize;
	Writer << UncompressedSize;
	Writer << FilenameLength;
	Writer << ExtraFieldLength;

	Writer.Serialize(TCHAR_TO_UTF8(*Path), FilenameLength);

	if (bCompress)
	{
		Writer.Serialize(CompressedData.GetData(), CompressedSize);
	}
	else
	{
		Writer.Serialize(const_cast<uint8*>(Data.GetData()), Data.Num());
	}

	Writer.InsertZeroed(OffsetOfCentralDirectory + LocalFileEntrySize + SizeOfCentralDirectory, 46 + FilenameLength);
	Writer.Seek(OffsetOfCentralDirectory + LocalFileEntrySize + SizeOfCentralDirectory);

	uint32 CentralDirectoryFileHeaderSignature = 0x02014b50;
	uint16 VersionMadeBy = 0;
	uint16 FileCommentLength = 0;
	uint16 DiskNumberWhereFileStarts = 0;
	uint16 InternalFileAttributes = 0;
	uint32 ExternalFileAttributes = 0;

	Writer << CentralDirectoryFileHeaderSignature;
	Writer << VersionMadeBy;
	Writer << VersionNeededToExtract;
	Writer << GeneralPurposeFlags;
	Writer << CompressionMethod;
	Writer << FileLastModificationTime;
	Writer << FileLastModificationDate;
	Writer << Crc;
	Writer << CompressedSize;
	Writer << UncompressedSize;
	Writer << FilenameLength;
	Writer << ExtraFieldLength;
	Writer << FileCommentLength;
	Writer << DiskNumberWhereFileStarts;
	Writer << InternalFileAttributes;
	Writer << ExternalFileAttributes;
	Writer << OffsetOfCentralDirectory;
	Writer.Serialize(TCHAR_TO_UTF8(*Path), FilenameLength);

	OffsetOfCentralDirectory += LocalFileEntrySize;
	SizeOfCentralDirectory += 46 + FilenameLength;
	TotalNumberOfCentralDirectoryRecords++;

	return true;
}

void FDMXZipper::AddEndOfCentralDirectory(FArrayWriter& Writer)
{
	uint32 EndOfCentralDirectoryRecordSignature = 0x06054b50;
	uint16 NumberOfThisDisk = 0;
	uint16 DiskWhereCentralDirectoryStarts = 0;
	uint16 CommentLength = 0;

	Writer << EndOfCentralDirectoryRecordSignature;
	Writer << NumberOfThisDisk;
	Writer << DiskWhereCentralDirectoryStarts;
	Writer << TotalNumberOfCentralDirectoryRecords; // on disk
	Writer << TotalNumberOfCentralDirectoryRecords;
	Writer << SizeOfCentralDirectory;
	Writer << OffsetOfCentralDirectory;
	Writer << CommentLength;
}

bool FDMXZipper::Parse()
{
	// Reset parser status
	Reader.Seek(0);
	OffsetsMap.Empty();
	NewFilesToShouldCompressMap.Empty();

	// Retrieve the trailer magic
	TArray<uint8> Magic;
	bool bIndexFound = false;
	int64 Index = 0;
	for (Index = Reader.Num() - 1; Index >= 0; Index--)
	{
		Magic.Insert(Reader[Index], 0);
		if (Magic.Num() == 4)
		{
			if (Magic[0] == 0x50 && Magic[1] == 0x4b && Magic[2] == 0x05 && Magic[3] == 0x06)
			{
				bIndexFound = true;
				break;
			}
			Magic.Pop();
		}
	}

	if (!bIndexFound)
	{
		return false;
	}

	uint16 DiskEntries = 0;
	uint16 TotalEntries = 0;
	uint32 ReaderSizeOfCentralDirectory = 0;
	uint32 ReaderOffsetOfCentralDirectory = 0;
	uint16 CommentLen = 0;

	constexpr uint64 TrailerMinSize = 22;
	constexpr uint64 CentralDirectoryMinSize = 46;

	if (Index + TrailerMinSize > Reader.Num())
	{
		return false;
	}

	// Skip signature and disk data
	Reader.Seek(Index + 8);
	Reader << DiskEntries;
	Reader << TotalEntries;
	Reader << ReaderSizeOfCentralDirectory;
	Reader << ReaderOffsetOfCentralDirectory;
	Reader << CommentLen;

	uint16 ReaderTotalNumberOfCentralDirectoryRecords = FMath::Min(DiskEntries, TotalEntries);

	for (uint16 DirectoryIndex = 0; DirectoryIndex < ReaderTotalNumberOfCentralDirectoryRecords; DirectoryIndex++)
	{
		if (ReaderTotalNumberOfCentralDirectoryRecords + CentralDirectoryMinSize > Reader.Num())
		{
			return false;
		}

		uint16 FilenameLen = 0;
		uint16 ExtraFieldLen = 0;
		uint16 EntryCommentLen = 0;
		uint32 EntryOffset = 0;

		// Seek to FilenameLen
		Reader.Seek(ReaderOffsetOfCentralDirectory + 28);
		Reader << FilenameLen;
		Reader << ExtraFieldLen;
		Reader << EntryCommentLen;
		// Seek to EntryOffset
		Reader.Seek(ReaderOffsetOfCentralDirectory + 42);
		Reader << EntryOffset;

		if (ReaderOffsetOfCentralDirectory + CentralDirectoryMinSize + FilenameLen + ExtraFieldLen + EntryCommentLen > Reader.Num())
		{
			return false;
		}

		TArray64<uint8> FilenameBytes;
		FilenameBytes.Append(Reader.GetData() + ReaderOffsetOfCentralDirectory + CentralDirectoryMinSize, FilenameLen);
		FilenameBytes.Add(0);

		OffsetsMap.Add(UTF8_TO_TCHAR(FilenameBytes.GetData()), EntryOffset);

		ReaderOffsetOfCentralDirectory += CentralDirectoryMinSize + FilenameLen + ExtraFieldLen + EntryCommentLen;
	}

	return true;
}
