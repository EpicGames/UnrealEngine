// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"


/**
 * Zip reader/writer for DMX specific Zip Files such as MVR and GDTF.
 *
 * Example of use:
 * bool DoStuffWithZip(const FString& ZipFileName)
 * {
 *		FDMXZipper Zipper;
 *		Zipper.LoadFromFile(ZipFileName);
 *
 *		Zipper.AddFile("hello/world", { 'A', 'B', 'C' });
 *		Zipper.AddFile("test001", { '0', '1', '2', '3', 'X' }, false);
 *		Zipper.AddFile("a/b/c/d/e/test002", { 80, 90, 100, 110, 111, 112, 113 });
 *
 *		TArray64<uint8> BigFile;
 *		if (FFileHelper::LoadFileToArray(BigFile, TEXT("D:/SM_MatPreviewMesh_01.stl")))
 *		{
 *			Zipper.AddFile("Meshes/Mesh.stl", BigFile);
 *		}
 *
 *		if (!Zipper.SaveToFile(ZipFileName))
 *		{
 *			return false;
 *		}
 *
 *		return true;
 * }
 */
class DMXZIP_API FDMXZipper
	: public TSharedFromThis<FDMXZipper>
{
public:
	/** Constructor */
	FDMXZipper();

	/** Loads the specified zip file */
	[[nodiscard]] bool LoadFromFile(const FString& Filename);

	/** Loads the data as zip file */
	[[nodiscard]] bool LoadFromData(const TArray64<uint8>& Data);

	/** Saves the zip to specified Filename */
	[[nodiscard]] bool SaveToFile(const FString& Filename);

	/** Gets the zip as data. Returns true on success */
	[[nodiscard]] bool GetData(TArray64<uint8>& OutData);

	/** Returns the File Names in the Zip. Note, may contain relative paths */
	TArray<FString> GetFiles() const;

	/**
	 * Adds a file to the Zip at its Relative File Path and Name.
	 * Note, files added this way need to exist on disk until SaveToFile is called.
	 * 
	 * Examples:
	 * AddFile("hello/world.txt", { 'A', 'B', 'C' });
	 * AddFile("test001", { '0', '1', '2', '3', 'X' }, false);
	 * AddFile("a/b/c/d/e/test002", { 80, 90, 100, 110, 111, 112, 113 });
	 */
	void AddFile(const FString& RelativeFilePathAndName, const TArray64<uint8>& Data, const bool bCompress = false);

	/** Gets the Data of a File in the Zip */
	[[nodiscard]] bool GetFileContent(const FString& Filename, TArray64<uint8>& OutData);

	/** Helper to unzip a file within the zip as temp file. Deletes the file when running out of scope. */
	struct DMXZIP_API FDMXScopedUnzipToTempFile
	{
		FDMXScopedUnzipToTempFile(const TSharedRef<FDMXZipper>& DMXZipper, const FString& FilenameInZip);
		~FDMXScopedUnzipToTempFile();

		/** Absolute File Path and Name of the Temp File */
		FString TempFilePathAndName;
	};

protected:
	/** Writes a zipped File to the array writer */
	[[nodiscard]] bool AddFileInternal(FArrayWriter& Writer, const FString& Path, const TArray64<uint8>& Data, const bool bCompress = true);

	/** Adds an End of Central Directory entry to the zip */
	void AddEndOfCentralDirectory(FArrayWriter& Writer);

	/** Parses the Zip File */
	[[nodiscard]] bool Parse();

	/** The Size of the Central Directory */
	uint32 SizeOfCentralDirectory = 0;

	/** The Offset of the Central Directory */
	uint32 OffsetOfCentralDirectory = 0;

	/** The total number of Central Directory records */
	uint16 TotalNumberOfCentralDirectoryRecords = 0;

	/** Array reader handling the data */
	FArrayReader Reader;

	/** Map of File Names and their Offset */
	TMap<FString, uint32> OffsetsMap;

	/** Map of Files added to the Zip */
	TMap<FString, TPair<TArray64<uint8>, bool>> NewFilesToShouldCompressMap;
};
